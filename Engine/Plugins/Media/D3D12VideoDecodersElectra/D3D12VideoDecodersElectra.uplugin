{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "0.1",
  "FriendlyName": "D3D12 Hardware accelerated video decoding plugin for the Electra media player",
  "Description": "Uses GPU vendor provided accelerators under Direct3D 12 Video",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "Category": "Media Players",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": false,
  "IsExperimentalVersion": false,
  "Installed": false,
  "Modules": [
    {
      "Name": "D3D12VideoDecodersElectra",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64" ],
      "TargetDenyList": [ "Server" ]
    }
  ],
  "Plugins": [
    {
      "Name": "ElectraCodecs",
      "Enabled": true
    }
  ]
}
