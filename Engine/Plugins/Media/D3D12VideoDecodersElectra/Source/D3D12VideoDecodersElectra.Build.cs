// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class D3D12VideoDecodersElectra : ModuleRules
	{
		protected virtual bool bIsDefaultDisabledOnPlatform
		{
			get
			{
				return false;
			}
		}
		protected virtual bool bIsDefaultIgnoredOnPlatform
		{
			get
			{
				return true;
			}
		}

		public D3D12VideoDecodersElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDefinitions.Add("ELECTRA_DECODERS_D3D12VIDEO_DISABLED_ON_PLATFORM=" + (bIsDefaultDisabledOnPlatform ? "1" : "0"));
			PublicDefinitions.Add("ELECTRA_DECODERS_D3D12VIDEO_IGNORED_ON_PLATFORM=" + (bIsDefaultIgnoredOnPlatform ? "1" : "0"));
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "RHI",
					"ElectraBase",
                    "ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders"
                });

			PrivateIncludePathModuleNames.Add("D3D12RHI");
		}
	}
}
