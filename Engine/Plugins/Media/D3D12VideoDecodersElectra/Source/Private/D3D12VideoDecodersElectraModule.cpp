// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12VideoDecodersElectraModule.h"
#include "VideoDecoder_D3D12.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "D3D12VideoDecodersElectraModule"

DEFINE_LOG_CATEGORY(LogD3D12VideoDecodersElectra);

class D3D12VideoDecodersElectraModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FD3D12VideoDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FD3D12VideoDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(D3D12VideoDecodersElectraModule, D3D12VideoDecodersElectra);

#undef LOCTEXT_NAMESPACE
