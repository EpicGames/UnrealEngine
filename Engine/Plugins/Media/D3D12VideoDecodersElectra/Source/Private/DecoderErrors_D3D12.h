// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_BAD_PARAMETER						1
#define ERRCODE_INTERNAL_ALREADY_CLOSED						2
#define ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD				3
#define ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER			5
#define ERRCODE_INTERNAL_FAILED_TO_DECODE					6
