// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoder_D3D12.h"
#include "VideoDecoder_D3D12_Common.h"

#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "RHI.h"
#include "ID3D12DynamicRHI.h"

#include "HAL/IConsoleManager.h"

#include "IElectraCodecFactory.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecRegistry.h"

#include "ElectraDecodersPlatformResources.h"
#include "D3D12VideoDecodersElectraModule.h"

// Codec specific implementations
#include "VideoDecoder_D3D12_H264.h"
#include "VideoDecoder_D3D12_H265.h"


namespace ElectraVideoDecodersD3D12Video
{
#ifndef ELECTRA_DECODERS_D3D12VIDEO_DISABLED_ON_PLATFORM
#define ELECTRA_DECODERS_D3D12VIDEO_DISABLED_ON_PLATFORM 1
#endif
#ifndef ELECTRA_DECODERS_D3D12VIDEO_IGNORED_ON_PLATFORM
#define ELECTRA_DECODERS_D3D12VIDEO_IGNORED_ON_PLATFORM 1
#endif

#if ELECTRA_DECODERS_D3D12VIDEO_DISABLED_ON_PLATFORM
static bool bDisableThisDecoder = true;
#else
static bool bDisableThisDecoder = false;
#endif

#if ELECTRA_DECODERS_D3D12VIDEO_IGNORED_ON_PLATFORM
static bool bDoNotUseThisDecoder = true;
#else
static bool bDoNotUseThisDecoder = false;
#endif

FAutoConsoleVariableRef CVarElectraDecoderD3D12VideoDisable(
	TEXT("ElectraDecoders.bDisableD3D12Video"),
	bDisableThisDecoder,
	TEXT("Globally disable the use of the D3D12 native video decoder"));

FAutoConsoleVariableRef CVarElectraDecoderD3D12DoNotUse(
	TEXT("ElectraDecoders.bDoNotUseD3D12Video"),
	bDoNotUseThisDecoder,
	TEXT("Do not use the D3D12 native video decoder on this platform"));

int32 FCodecFormatHelper::FindSupportedFormats(ID3D12Device* InD3D12Device)
{
	HRESULT Result;
	CodecInfos.Empty();
	DxVideoDevice = nullptr;

	ID3D12Device* DxDevice = InD3D12Device;
	if (!DxDevice)
	{
		// Not D3D 12, nothing to do.
		if (RHIGetInterfaceType() != ERHIInterfaceType::D3D12)
		{
			return 0;
		}
		DxDevice = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
	}
	if (!DxDevice)
	{
		return 0;
	}

	// Is this device a video capable device?
	TRefCountPtr<ID3D12VideoDevice> VideoDevice;
	if ((Result = DxDevice->QueryInterface(__uuidof(ID3D12VideoDevice), (void**)VideoDevice.GetInitReference())) != S_OK)
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Log, "The current RHI device is not a video decoding capable device.");
		return 0;
	}

	UINT NumNodes = DxDevice->GetNodeCount();
	if (NumNodes == 0)
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Log, "The current RHI device reports zero nodes and cannot be used.");
		return 0;
	}
	else if (NumNodes > 1)
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Log, "The current RHI device reports %u nodes. Using node index 0", NumNodes);
	}
	UINT NodeIndex = 0;

	D3D12_FEATURE_DATA_VIDEO_DECODE_PROFILE_COUNT ProfileCount {};
	ProfileCount.NodeIndex = NodeIndex;
	if ((Result = VideoDevice->CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_PROFILE_COUNT, &ProfileCount, sizeof(ProfileCount))) != S_OK)
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Error, "CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_PROFILE_COUNT) failed with 0x%08x", Result);
		return 0;
	}

	// Get all supported profiles
	TArray<GUID> ProfileGUIDs;
	ProfileGUIDs.SetNum(ProfileCount.ProfileCount);
	D3D12_FEATURE_DATA_VIDEO_DECODE_PROFILES Profiles {};
	Profiles.NodeIndex = NodeIndex;
	Profiles.ProfileCount = ProfileCount.ProfileCount;
	Profiles.pProfiles = ProfileGUIDs.GetData();
	if ((Result = VideoDevice->CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_PROFILES, &Profiles, sizeof(Profiles))) != S_OK)
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Error, "CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_PROFILES) failed with 0x%08x", Result);
		return 0;
	}

	// Iterate the profiles and handle those we are interested in.
	for(int32 nProf=0; nProf<ProfileGUIDs.Num(); ++nProf)
	{
		if (ProfileGUIDs[nProf] != D3D12_VIDEO_DECODE_PROFILE_H264 &&
			ProfileGUIDs[nProf] != D3D12_VIDEO_DECODE_PROFILE_HEVC_MAIN &&
			ProfileGUIDs[nProf] != D3D12_VIDEO_DECODE_PROFILE_HEVC_MAIN10 &&
			ProfileGUIDs[nProf] != D3D12_VIDEO_DECODE_PROFILE_VP9 &&
			ProfileGUIDs[nProf] != D3D12_VIDEO_DECODE_PROFILE_VP9_10BIT_PROFILE2)
		{
			continue;
		}

		FCodecInfo Info;
		Info.ProfileGUID = ProfileGUIDs[nProf];
		if (ProfileGUIDs[nProf] == D3D12_VIDEO_DECODE_PROFILE_H264)
		{
			Info.CodecType = ECodecType::H264;
		}
		else if (ProfileGUIDs[nProf] == D3D12_VIDEO_DECODE_PROFILE_HEVC_MAIN)
		{
			Info.CodecType = ECodecType::H265;
		}
		else if (ProfileGUIDs[nProf] == D3D12_VIDEO_DECODE_PROFILE_HEVC_MAIN10)
		{
			Info.CodecType = ECodecType::H265;
			Info.b10Bit = true;
		}
		else if (ProfileGUIDs[nProf] == D3D12_VIDEO_DECODE_PROFILE_VP9)
		{
			Info.CodecType = ECodecType::VP9;
		}
		else if (ProfileGUIDs[nProf] == D3D12_VIDEO_DECODE_PROFILE_VP9_10BIT_PROFILE2)
		{
			Info.CodecType = ECodecType::VP9;
			Info.b10Bit = true;
		}
		else
		{
			check(!"This wasn't skipped above!");
			continue;
		}

		D3D12_VIDEO_DECODE_CONFIGURATION DecodeConfiguration {};
		DecodeConfiguration.BitstreamEncryption = D3D12_BITSTREAM_ENCRYPTION_TYPE_NONE;
		DecodeConfiguration.InterlaceType = D3D12_VIDEO_FRAME_CODED_INTERLACE_TYPE_NONE;
		DecodeConfiguration.DecodeProfile = ProfileGUIDs[nProf];

		// Get number of supported pixel formats
		D3D12_FEATURE_DATA_VIDEO_DECODE_FORMAT_COUNT FormatCount {};
		FormatCount.NodeIndex = NodeIndex;
		FormatCount.Configuration = DecodeConfiguration;
		if ((Result = VideoDevice->CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_FORMAT_COUNT, &FormatCount, sizeof(FormatCount))) != S_OK)
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Error, "CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_FORMAT_COUNT) failed with 0x%08x", Result);
			return 0;
		}
		// Get supported pixel formats
		D3D12_FEATURE_DATA_VIDEO_DECODE_FORMATS Formats {};
		Formats.NodeIndex = NodeIndex;
		Formats.Configuration = DecodeConfiguration;
		Formats.FormatCount = FormatCount.FormatCount;
		TArray<DXGI_FORMAT> PixFmts;
		PixFmts.SetNum(FormatCount.FormatCount);
		Formats.pOutputFormats = PixFmts.GetData();
		if ((Result = VideoDevice->CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_FORMATS, &Formats, sizeof(Formats))) != S_OK)
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Error, "CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_FORMATS) failed with 0x%08x", Result);
			return 0;
		}
		// Only use common pixel formats, nothing obscure please.
		for(int32 nPixFmt=0; nPixFmt<PixFmts.Num(); ++nPixFmt)
		{
			if (PixFmts[nPixFmt] == DXGI_FORMAT_NV12 ||
				PixFmts[nPixFmt] == DXGI_FORMAT_P010 ||
				PixFmts[nPixFmt] == DXGI_FORMAT_P016)
			{
				Info.PixelFormats.Emplace(PixFmts[nPixFmt]);
			}
		}

		// Is this usable?
		if (Info.PixelFormats.Num())
		{
			// Do an AddUnique here since we have seen the exact same profile being reported more than once.
			// Note: the == comparison of FCodecInfo doesn't handle different ordering of the pixel formats,
			//       so if those were shuffled in the profiles we could end up with more than one, but that
			//       is not an actual problem.
			CodecInfos.AddUnique(Info);
		}
	}
	// If we have anything supported we remember the video device for later.
	if (CodecInfos.Num())
	{
		DxVideoDevice = VideoDevice;
		DxDeviceNodeIndex = NodeIndex;
	}
	return CodecInfos.Num();
}

const FCodecFormatHelper::FCodecInfo* FCodecFormatHelper::HaveFormat(ECodecType InType, int32 InNumBits)
{
	for(int32 i=0, iMax=CodecInfos.Num(); i<iMax; ++i)
	{
		if (CodecInfos[i].CodecType == InType &&
			((InNumBits == 8 && CodecInfos[i].b10Bit == false) || (InNumBits == 10 && CodecInfos[i].b10Bit == true)))
		{
			return &CodecInfos[i];
		}
	}
	return nullptr;
}

FDecodedPictureBuffer::~FDecodedPictureBuffer()
{
	check(Frames.IsEmpty());
	ReleaseAllFrames(0);
}

void FDecodedPictureBuffer::ReleaseAllFrames(int32 InWaitForEachFrameMillis)
{
	// Drop all frames in the available queue. These are shared with the
	// frames list so that is safe.
	AvailableQueue.Empty();
	while(!Frames.IsEmpty())
	{
		TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> Frame = Frames.Pop();
		check(Frame.IsValid());
		if (Frame.IsValid())
		{
			if (InWaitForEachFrameMillis)
			{
				Frame->Sync.AwaitCompletion(InWaitForEachFrameMillis);
			}
			Frame->Texture.SafeRelease();
		}
		Frame.Reset();
	}
}

TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> FDecodedPictureBuffer::GetFrameAtIndex(int32 InIndex)
{
	check(InIndex >= 0 && InIndex < Frames.Num());
	return InIndex >= 0 && InIndex < Frames.Num() ? Frames[InIndex] : nullptr;
}

TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> FDecodedPictureBuffer::GetFrameForResource(const ID3D12Resource* InResource)
{
	for(auto& it : Frames)
	{
		if (it->Texture.GetReference() == InResource)
		{
			return it;
		}
	}
	return nullptr;
}

TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> FDecodedPictureBuffer::GetNextUnusedFrame()
{
	return AvailableQueue.Num() ? AvailableQueue.Pop() : nullptr;
}

void FDecodedPictureBuffer::ReturnUnusedFrameToAvailableQueue(TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>&& InFrame)
{
	if (InFrame.IsValid())
	{
		AvailableQueue.Push(MoveTemp(InFrame));
	}
}

void FDecodedPictureBuffer::ReturnFrameToAvailableQueue(TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>&& InFrame)
{
	if (InFrame.IsValid())
	{
		AvailableQueue.Insert(MoveTemp(InFrame), 0);
	}
}


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FD3D12VideoDecoderFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FD3D12VideoDecoderFactory, ESPMode::ThreadSafe>
{
private:
	TUniquePtr<FCodecFormatHelper> CurrentFormats;
	FCriticalSection AccessLock;

public:
	FD3D12VideoDecoderFactory(TUniquePtr<FCodecFormatHelper> InCurrentFormats)
		: CurrentFormats(MoveTemp(InCurrentFormats))
	{ }
	virtual ~FD3D12VideoDecoderFactory()
	{}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	static void GetPlatformConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		// TBD
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)8));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
		OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
		OutOptions.Emplace(IElectraDecoderFeature::StartcodeToLength, FVariant(int32(0)));
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		GetPlatformConfigurationOptions(OutOptions);
	}

	const FCodecFormatHelper::FCodecInfo* GetFormatIfSupported(D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& OutSupport, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return nullptr;
		}
		const Electra::FCodecTypeFormat::FVideo& vi = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		int32 Width = (int32) vi.Width;
		int32 Height = (int32) vi.Height;
		int64 bps = InCodecFormat.Bitrate ? InCodecFormat.Bitrate : InCodecFormat.AverageBitrate ? InCodecFormat.AverageBitrate : 0;
		int64 fps_n = vi.FrameRate.IsValid() ? vi.FrameRate.Numerator : 0;
		uint32 fps_d = vi.FrameRate.IsValid() ? vi.FrameRate.Denominator : 0;

		// Get properties that are not conveyed in the codec type format structure but in the additional options.
		// Look for max width. If it is set the others should also be set.
		int32 MaxWidth = (int32)ElectraDecodersUtil::GetVariantValueSafeI64(InAdditionalOptions, TEXT("max_width"), 0);
		if (MaxWidth > 0)
		{
			Width = MaxWidth;
			Height = (int32)ElectraDecodersUtil::GetVariantValueSafeI64(InAdditionalOptions, TEXT("max_height"), 0);
			bps = ElectraDecodersUtil::GetVariantValueSafeI64(InAdditionalOptions, TEXT("max_bitrate"), 0);
			fps_n = ElectraDecodersUtil::GetVariantValueSafeI64(InAdditionalOptions, TEXT("max_fps_n"), 0);
			fps_d = (uint32)ElectraDecodersUtil::GetVariantValueSafeI64(InAdditionalOptions, TEXT("max_fps_d"), 0);
		}

		const FCodecFormatHelper::FCodecInfo* Codec = nullptr;
		D3D12_VIDEO_DECODE_CONFIGURATION DecodeConfiguration {};
		DecodeConfiguration.BitstreamEncryption = D3D12_BITSTREAM_ENCRYPTION_TYPE_NONE;
		DecodeConfiguration.InterlaceType = D3D12_VIDEO_FRAME_CODED_INTERLACE_TYPE_NONE;
		if (InCodecFormat.RFC6381.StartsWith(TEXT("avc"), ESearchCase::IgnoreCase))
		{
			// Only support Baseline, Main and High profile.
			if (vi.Profile.Profile == 66 || vi.Profile.Profile == 77 || vi.Profile.Profile == 100)
			{
				Codec = CurrentFormats->HaveFormat(FCodecFormatHelper::ECodecType::H264, 8);
			}
		}
		else if (InCodecFormat.RFC6381.StartsWith(TEXT("hvc"), ESearchCase::IgnoreCase) || InCodecFormat.RFC6381.StartsWith(TEXT("hev"), ESearchCase::IgnoreCase))
		{
			// The DXVA2 structure `DXVA_PicParams_HEVC` has fixed sizes for `column_width_minus1`
			// and `row_height_minus1` that allow for at most level 6.3. Any higher level using tiles
			// *could* use more than that and is hence not decodable.
			if (vi.Profile.Level > 189 /*6*30+3*3*/)
			{
				return nullptr;
			}
			// ITU-T H.265 only specifies profile space 0.
			if (vi.Profile.ProfileSpace != 0)
			{
				return nullptr;
			}

			// Main profile (8 bit)
			if (vi.Profile.Profile == 1)
			{
				Codec = CurrentFormats->HaveFormat(FCodecFormatHelper::ECodecType::H265, 8);
			}
			// Main10 profile (10 bit)
			else if (vi.Profile.Profile == 2)
			{
				Codec = CurrentFormats->HaveFormat(FCodecFormatHelper::ECodecType::H265, 10);
			}
			else
			{
				// Not supported.
				return nullptr;
			}
		}
		else
		{
			// Not supported at the moment.
			return nullptr;
		}

		if (Codec)
		{
			DecodeConfiguration.DecodeProfile = Codec->ProfileGUID;

			OutSupport.NodeIndex = CurrentFormats->GetVideoDeviceNodeIndex();
			OutSupport.Configuration = DecodeConfiguration;
			OutSupport.Width = Width;
			OutSupport.Height = Height;
			// Use the first format that was reported back. We assume this is the best one possible.
			OutSupport.DecodeFormat = Codec->PixelFormats[0];
			OutSupport.FrameRate.Numerator = (uint32)fps_n;
			OutSupport.FrameRate.Denominator = fps_n && fps_d ? fps_d : 0;
			OutSupport.BitRate = (uint32)bps;
			HRESULT Result = CurrentFormats->GetVideoDevice()->CheckFeatureSupport(D3D12_FEATURE_VIDEO_DECODE_SUPPORT, &OutSupport, sizeof(OutSupport));
			if (Result == S_OK && (OutSupport.SupportFlags & D3D12_VIDEO_DECODE_SUPPORT_FLAG_SUPPORTED) == D3D12_VIDEO_DECODE_SUPPORT_FLAG_SUPPORTED)
			{
				// For simplicity we require decode tier 2
				if (OutSupport.DecodeTier != D3D12_VIDEO_DECODE_TIER_2)
				{
					UE_LOGF(LogD3D12VideoDecodersElectra, Log, "Decode tier 2 is needed, but tier %d was returned.", (int)OutSupport.DecodeTier);
					Codec = nullptr;
				}

				// We don't support reference only allocations yet
				if ((OutSupport.ConfigurationFlags & D3D12_VIDEO_DECODE_CONFIGURATION_FLAG_REFERENCE_ONLY_ALLOCATIONS_REQUIRED) != 0)
				{
					UE_LOGF(LogD3D12VideoDecodersElectra, Log, "Decode reference only allocations are not supported, but configuration flags 0x%08x were returned.", (int)OutSupport.ConfigurationFlags);
					Codec = nullptr;
				}

				// Do a custom platform capability check.
				if (!FD3D12VideoDecoder::CheckPlatformDecodeCapabilities(OutSupport, InCodecFormat, InAdditionalOptions))
				{
					UE_LOGF(LogD3D12VideoDecodersElectra, Log, "Platform rejected decoding of %d*%d @ %d/%d fps", (int)Width, (int)Height, (int)fps_n, (int)fps_d);
					Codec = nullptr;
				}
			}
			else
			{
				UE_LOGF(LogD3D12VideoDecodersElectra, Log, "Decoding of %d*%d @ %d/%d fps is not supported", (int)Width, (int)Height, (int)fps_n, (int)fps_d);
				Codec = nullptr;
			}
		}
		return Codec;
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (bDoNotUseThisDecoder)
		{
			return 0;
		}
		// No formats, no support.
		if (!CurrentFormats.IsValid() || !CurrentFormats->GetVideoDevice().IsValid())
		{
			return 0;
		}

		D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT DecodeSupport {};
		const FCodecFormatHelper::FCodecInfo* Codec = GetFormatIfSupported(DecodeSupport, InCodecFormat, InAdditionalOptions);
		return Codec ? 5 : 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		if (bDoNotUseThisDecoder)
		{
			return nullptr;
		}

		HRESULT Result;

		// Do this under lock as it may be possible that the D3D device changed and we have to rebuild the codec list.
		FScopeLock lock(&AccessLock);

		// Verify that we are using the same D3D device that was used in the initial format determination.
		TRefCountPtr<ID3D12Device> D3DDevice;
		int32 D3DDeviceVersion = 0;
		if (!FElectraDecodersPlatformResources::GetD3DDeviceAndVersion((void**)D3DDevice.GetInitReference(), &D3DDeviceVersion))
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Error, "Could not obtain the current RHI D3D device.");
			return nullptr;
		}
		// Must be a D3D12 device.
		if (D3DDeviceVersion != 12000)
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Warning, "Current RHI D3D device is not a D3D12 device.");
			return nullptr;
		}

		bool bRedoFromStart = false;
		if (CurrentFormats.IsValid())
		{
			TRefCountPtr<ID3D12VideoDevice> VideoDevice;
			if ((Result = D3DDevice->QueryInterface(__uuidof(ID3D12VideoDevice), (void**)VideoDevice.GetInitReference())) != S_OK)
			{
				UE_LOGF(LogD3D12VideoDecodersElectra, Warning, "The current RHI device is not a video decoding capable device.");
				return nullptr;
			}
			// Not the same video device any more?
			if (VideoDevice != CurrentFormats->GetVideoDevice())
			{
				bRedoFromStart = true;
			}
		}
		else
		{
			bRedoFromStart = true;
		}
		// Start over determining the supported formats?
		if (bRedoFromStart)
		{
			if (!CurrentFormats.IsValid())
			{
				CurrentFormats = MakeUnique<FCodecFormatHelper>();
			}
			CurrentFormats->FindSupportedFormats(D3DDevice.GetReference());
		}

		D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT DecodeSupport {};
		const FCodecFormatHelper::FCodecInfo* Codec = GetFormatIfSupported(DecodeSupport, InCodecFormat, InAdditionalOptions);
		check(Codec);
		if (!Codec)
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Log, "The current RHI device does not support decoding this format.");
			return nullptr;
		}
		TSharedPtr<FD3D12VideoDecoder, ESPMode::ThreadSafe> New;
		switch(Codec->CodecType)
		{
			case FCodecFormatHelper::ECodecType::H264:
			{
				New = MakeShared<FD3D12VideoDecoder_H264>(*Codec, DecodeSupport, InAdditionalOptions, D3DDevice, CurrentFormats->GetVideoDevice(), CurrentFormats->GetVideoDeviceNodeIndex());
				break;
			}
			case FCodecFormatHelper::ECodecType::H265:
			{
				New = MakeShared<FD3D12VideoDecoder_H265>(*Codec, DecodeSupport, InAdditionalOptions, D3DDevice, CurrentFormats->GetVideoDevice(), CurrentFormats->GetVideoDeviceNodeIndex());
				break;
			}
		}
		if (New.IsValid())
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Verbose, "Created a D3D12 video decoder.");
		}
		return New;
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("D3D12 Video")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString(TEXT("Microsoft"))); }
};







FD3D12VideoDecoder::FD3D12VideoDecoder(const FCodecFormatHelper::FCodecInfo& InCodecInfo, const D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InDecodeSupport, const TMap<FString, FVariant>& InOptions, const TRefCountPtr<ID3D12Device>& InD3D12Device, const TRefCountPtr<ID3D12VideoDevice>& InVideoDevice, uint32 InVideoDeviceNodeIndex)
	: CodecInfo(InCodecInfo)
	, DecodeSupport(InDecodeSupport)
	, InitialCreationOptions(InOptions)
	, D3D12Device(InD3D12Device)
	, VideoDevice(InVideoDevice)
	, VideoDeviceNodeIndex(InVideoDeviceNodeIndex)
{
}

FD3D12VideoDecoder::~FD3D12VideoDecoder()
{
	// Note: It is the codec specific implementation's responsibility to check that the decoder
	//       has already been closed and/or do it. When we get here we cannot call into any
	//       derived classes methods any more as it has already been destroyed and the vtable
	//       has become invalid.
	VideoDevice.SafeRelease();
	D3D12Device.SafeRelease();
}

void FD3D12VideoDecoder::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	FD3D12VideoDecoderFactory::GetPlatformConfigurationOptions(OutFeatures);
}

void FD3D12VideoDecoder::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}


bool FD3D12VideoDecoder::ResetToCleanStart()
{
	ReturnAllFrames();

	if (VideoDecoderSync.IsValid())
	{
		VideoDecoderSync->AwaitCompletion(500);
	}

	if (DPB.IsValid())
	{
		// Return the "missing" frame if the decoder had to create one.
		DPB->ReturnFrameToAvailableQueue(MoveTemp(MissingReferenceFrame));
		DPB->ReleaseAllFrames(500);
		DPB.Reset();
	}

	// Codec specific reset.
	InternalResetToCleanStart();

	RunningFrameNumLo = 0;
	RunningFrameNumHi = 1;
	bIsDraining = false;

	TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe> fdr;
	while(AvailableFrameDecodeResourceQueue.Dequeue(fdr))
	{
		fdr->D3DDecoder.SafeRelease();
		fdr->D3DDecoderHeap.SafeRelease();
	}
	VideoDecoder.SafeRelease();
	VideoDecoderCommandList.SafeRelease();
	VideoDecoderCommandAllocator.SafeRelease();
	VideoDecoderCommandQueue.SafeRelease();
	VideoDecoderSync.Reset();

	CurrentConfig.Reset();
	StatusReportFeedbackNumber = 0;
	return true;
}

void FD3D12VideoDecoder::ReturnAllFrames()
{
	auto ReturnFrames = [](TArray<TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe>>& InList) -> void
	{
		while(!InList.IsEmpty())
		{
			if (InList[0]->OwningDPB.IsValid())
			{
				InList[0]->OwningDPB->ReturnFrameToAvailableQueue(MoveTemp(InList[0]->DecodedFrame));
				InList[0]->OwningDPB.Reset();
			}
			InList.RemoveAt(0);
		}
	};
	ReturnFrames(FramesGivenOutForOutput);
	ReturnFrames(FramesReadyForOutput);
	ReturnFrames(FramesInDecoder);
	if (DPB.IsValid())
	{
		check(DPB->AvailableQueue.Num() + (MissingReferenceFrame.IsValid() ? 1 : 0) == DPB->Frames.Num());
	}
}




IElectraDecoder::EDecoderError FD3D12VideoDecoder::ExecuteCommonDecode(const D3D12_VIDEO_DECODE_INPUT_STREAM_ARGUMENTS& InInputArgs, const D3D12_VIDEO_DECODE_OUTPUT_STREAM_ARGUMENTS& InOutputArgs)
{
	HRESULT Result;

	// All frames, the reference frames as well as the output frame are given in the list of reference frames.
	// We can use that list to check all the frame's fences for readiness.
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> TargetFrame;
	for(uint32 i=0; i<InInputArgs.ReferenceFrames.NumTexture2Ds; ++i)
	{
		if (InInputArgs.ReferenceFrames.ppTexture2Ds[i])
		{
			check(InInputArgs.ReferenceFrames.pSubresources == nullptr || InInputArgs.ReferenceFrames.pSubresources[i] == 0);

			TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> Frame = DPB->GetFrameForResource(InInputArgs.ReferenceFrames.ppTexture2Ds[i]);
			check(Frame.IsValid());
			if (!Frame.IsValid())
			{
				PostError(0, TEXT("ExecuteCommonDecode() did not find resource in reference frame list in the DPB"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
			if (InInputArgs.ReferenceFrames.ppTexture2Ds[i] == InOutputArgs.pOutputTexture2D)
			{
				TargetFrame = Frame;
			}

			// Wait for the fence of the frame to be signaled.
			// Do this with a timeout in case the outside code that works with these frames is stuck.
			if (!Frame->Sync.AwaitCompletion(100))
			{
				UE_LOGF(LogD3D12VideoDecodersElectra, Warning, "ExecuteCommonDecode() waited too long for a reference frame fence to be signaled. Trying again later.");
				return IElectraDecoder::EDecoderError::NoBuffer;
			}
		}
	}
	// Check that the output frame really was in the list.
	check(TargetFrame.IsValid());

	if ((Result = VideoDecoderCommandAllocator->Reset()) != S_OK)
	{
		PostError(Result, TEXT("ExecuteCommonDecode() failed to reset command allocator"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}
	if ((Result = VideoDecoderCommandList->Reset(VideoDecoderCommandAllocator)) != S_OK)
	{
		PostError(Result, TEXT("ExecuteCommonDecode() failed to reset command list"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}

	const int32 kMaxBarriers = FFrameDecodeResource::kMaxRefFrames * 2;
	check(InInputArgs.ReferenceFrames.NumTexture2Ds + 1 <= kMaxBarriers);
	if (InInputArgs.ReferenceFrames.NumTexture2Ds + 1 > kMaxBarriers)
	{
		PostError(0, TEXT("ExecuteCommonDecode() out of barriers"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}

	TArray<ID3D12Resource*> TransitionedResources;
	D3D12_RESOURCE_BARRIER Barriers[kMaxBarriers] {};
	uint32 NumBarriers = 1;
	// Transition the target frame to video-decode-write
	Barriers[0].Type = D3D12_RESOURCE_BARRIER_TYPE_TRANSITION;
	Barriers[0].Flags = D3D12_RESOURCE_BARRIER_FLAG_NONE;
	Barriers[0].Transition.pResource = InOutputArgs.pOutputTexture2D;
	Barriers[0].Transition.Subresource = D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES;
	Barriers[0].Transition.StateBefore = D3D12_RESOURCE_STATE_COMMON;
	Barriers[0].Transition.StateAfter  = D3D12_RESOURCE_STATE_VIDEO_DECODE_WRITE;
	TransitionedResources.Emplace(InOutputArgs.pOutputTexture2D);
	for(uint32 i=0; i<InInputArgs.ReferenceFrames.NumTexture2Ds; ++i)
	{
		// Transition the reference frames to video-decode-read. Check that we do not transition the same resource
		// more than once in case it appears in multiple reference frame slots.
		if (InInputArgs.ReferenceFrames.ppTexture2Ds[i] && !TransitionedResources.Contains(InInputArgs.ReferenceFrames.ppTexture2Ds[i]))
		{
			Barriers[NumBarriers].Type = D3D12_RESOURCE_BARRIER_TYPE_TRANSITION;
			Barriers[NumBarriers].Flags = D3D12_RESOURCE_BARRIER_FLAG_NONE;
			Barriers[NumBarriers].Transition.pResource = InInputArgs.ReferenceFrames.ppTexture2Ds[i];
			Barriers[NumBarriers].Transition.Subresource = D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES;
			Barriers[NumBarriers].Transition.StateBefore = D3D12_RESOURCE_STATE_COMMON;
			Barriers[NumBarriers].Transition.StateAfter  = D3D12_RESOURCE_STATE_VIDEO_DECODE_READ;
			TransitionedResources.Emplace(InInputArgs.ReferenceFrames.ppTexture2Ds[i]);
			++NumBarriers;
		}
	}
	VideoDecoderCommandList->ResourceBarrier(NumBarriers, Barriers);
	VideoDecoderCommandList->DecodeFrame(VideoDecoder, &InOutputArgs, &InInputArgs);
	// Reverse the transitions
	for(uint32 i=0; i<NumBarriers; ++i)
	{
		Barriers[i].Transition.StateBefore = Barriers[i].Transition.StateAfter;
		Barriers[i].Transition.StateAfter  = D3D12_RESOURCE_STATE_COMMON;
	}
	VideoDecoderCommandList->ResourceBarrier(NumBarriers, Barriers);
	if ((Result = VideoDecoderCommandList->Close()) != S_OK)
	{
		PostError(Result, TEXT("ExecuteCommonDecode() closing command list failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}

	// Execute the command list
	ID3D12CommandList* dcmdl[1] = { VideoDecoderCommandList.GetReference() };
	VideoDecoderCommandQueue->ExecuteCommandLists(1, dcmdl);
	if ((Result = VideoDecoderCommandQueue->Signal(TargetFrame->Sync.GetFence(), TargetFrame->Sync.IncrementAndGetNewFenceValue())) != S_OK)
	{
		PostError(Result, TEXT("ExecuteCommonDecode() signaling target frame fence in command queue failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}
	if ((Result = VideoDecoderCommandQueue->Signal(VideoDecoderSync->GetFence(), VideoDecoderSync->IncrementAndGetNewFenceValue())) != S_OK)
	{
		PostError(Result, TEXT("ExecuteCommonDecode() signaling decoder fence in command queue failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}

	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EOutputStatus FD3D12VideoDecoder::HaveOutput()
{
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	if (FramesReadyForOutput.Num())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	if (!VideoDecoder.IsValid())
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}
	if (bIsDraining)
	{
		bIsDraining = false;
		ReturnAllFrames();
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FD3D12VideoDecoder::GetOutput()
{
	if (FramesReadyForOutput.Num())
	{
		auto Out = FramesReadyForOutput[0];
		FramesReadyForOutput.RemoveAt(0);
		FramesGivenOutForOutput.Emplace(Out);
		return Out;
	}
	return nullptr;
}


bool FD3D12VideoDecoder::InternalDecoderCreate()
{
	check(D3D12Device.IsValid() && VideoDevice.IsValid());
	if (!D3D12Device.IsValid() || !VideoDevice.IsValid())
	{
		return PostError(0, TEXT("No D3D video device set"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	check(VideoDeviceNodeIndex == DecodeSupport.NodeIndex);
	const uint32 VideoDeviceNodeMask = GetNodeMask();

	HRESULT Result;

	TUniquePtr<FSyncObject> NewSync = MakeUnique<FSyncObject>();
	if ((Result = NewSync->Create(D3D12Device, 0)) != S_OK)
	{
		return PostError(Result, TEXT("Creating sync object"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	TRefCountPtr<ID3D12CommandQueue> NewCommandQueue;
	D3D12_COMMAND_QUEUE_DESC CommandQueueDesc {};
	CommandQueueDesc.Type = D3D12_COMMAND_LIST_TYPE_VIDEO_DECODE;
	CommandQueueDesc.NodeMask = VideoDeviceNodeMask;
	if ((Result = D3D12Device->CreateCommandQueue(&CommandQueueDesc, __uuidof(ID3D12CommandQueue), (void**)NewCommandQueue.GetInitReference())) != S_OK)
	{
		return PostError(Result, TEXT("CreateCommandQueue() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	TRefCountPtr<ID3D12CommandAllocator> NewCommandAllocator;
	if ((Result = D3D12Device->CreateCommandAllocator(D3D12_COMMAND_LIST_TYPE_VIDEO_DECODE, __uuidof(ID3D12CommandAllocator), (void**)NewCommandAllocator.GetInitReference())) != S_OK)
	{
		return PostError(Result, TEXT("CreateCommandAllocator() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	TRefCountPtr<ID3D12VideoDecodeCommandList> NewDecodeCommandList;
	if ((Result = D3D12Device->CreateCommandList(VideoDeviceNodeMask, D3D12_COMMAND_LIST_TYPE_VIDEO_DECODE, NewCommandAllocator.GetReference(), nullptr, __uuidof(ID3D12VideoDecodeCommandList), (void**)NewDecodeCommandList.GetInitReference())) != S_OK)
	{
		return PostError(Result, TEXT("CreateCommandList() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}
	if ((Result = NewDecodeCommandList->Close()) != S_OK)
	{
		return PostError(Result, TEXT("CommandList->Close() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}
	ID3D12CommandList* CommandLists[1] = { NewDecodeCommandList.GetReference() };
	NewCommandQueue->ExecuteCommandLists(1, CommandLists);
	if ((Result = NewCommandQueue->Signal(NewSync->GetFence(), NewSync->IncrementAndGetNewFenceValue())) != S_OK)
	{
		return PostError(Result, TEXT("CommandQueue->Signal() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}
	bool bOk = NewSync->AwaitCompletion(INFINITE);
	check(bOk);

	TRefCountPtr<ID3D12VideoDecoder> NewDecoder;
	D3D12_VIDEO_DECODER_DESC DecoderDesc {};
	DecoderDesc.NodeMask = VideoDeviceNodeMask;
	DecoderDesc.Configuration = DecodeSupport.Configuration;
	if ((Result = VideoDevice->CreateVideoDecoder(&DecoderDesc, __uuidof(ID3D12VideoDecoder), (void**)NewDecoder.GetInitReference())) != S_OK)
	{
		return PostError(Result, TEXT("CreateVideoDecoder() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	VideoDecoderSync = MoveTemp(NewSync);
	VideoDecoderCommandQueue = NewCommandQueue;
	VideoDecoderCommandAllocator = NewCommandAllocator;
	VideoDecoderCommandList = NewDecodeCommandList;
	VideoDecoder = NewDecoder;
	return true;
}

bool FD3D12VideoDecoder::CreateDecoderHeap(int32 InDPBSize, int32 InMaxWidth, int32 InMaxHeight, int32 InImageSizeAlignment)
{
	if (InDPBSize <= 0)
	{
		return PostError(0, TEXT("DPB size is invalid"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	HRESULT Result;
	TRefCountPtr<ID3D12VideoDecoderHeap> NewHeap;
	D3D12_VIDEO_DECODER_HEAP_DESC HeapDesc {};
	HeapDesc.NodeMask = GetNodeMask();
	HeapDesc.Configuration = DecodeSupport.Configuration;
	const uint32 Alignment = (uint32) InImageSizeAlignment;
	const uint32 AlignedWidth = Align(InMaxWidth, Alignment);
	const uint32 AlignedHeight = Align(InMaxHeight, Alignment);
#if 1
	HeapDesc.DecodeWidth = AlignedWidth;
	HeapDesc.DecodeHeight = AlignedHeight;
#else
	HeapDesc.DecodeWidth = DecodeSupport.Width;
	HeapDesc.DecodeHeight = DecodeSupport.Height;
#endif
	check(!CodecInfo.PixelFormats.IsEmpty());
	HeapDesc.Format = CodecInfo.PixelFormats[0];
	// best not to set those
	//HeapDesc.FrameRate = DecodeSupport.FrameRate;
	//HeapDesc.BitRate = DecodeSupport.BitRate;
	HeapDesc.MaxDecodePictureBufferCount = InDPBSize;
	if ((Result = VideoDevice->CreateVideoDecoderHeap(&HeapDesc, __uuidof(ID3D12VideoDecoderHeap), (void**)NewHeap.GetInitReference())) != S_OK)
	{
		return PostError(Result, TEXT("CreateVideoDecoderHeap() failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
	}
	CurrentConfig.VideoDecoderHeap = NewHeap;
	CurrentConfig.VideoDecoderDPBWidth = InMaxWidth;
	CurrentConfig.VideoDecoderDPBHeight = InMaxHeight;
	CurrentConfig.MaxNumInDPB = InDPBSize;
	return true;
}


bool FD3D12VideoDecoder::CreateDPB(TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe>& OutDPB, int32 InMaxWidth, int32 InMaxHeight, int32 InImageSizeAlignment, int32 InNumFrames)
{
	if (InNumFrames <= 0)
	{
		return PostError(0, TEXT("Bad number of frames"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}
	if (InNumFrames > FFrameDecodeResource::kMaxRefFrames)
	{
		return PostError(0, TEXT("Too many frames requested than fit into managing structure."), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe> newdpb = MakeShared<FDecodedPictureBuffer, ESPMode::ThreadSafe>();
	newdpb->Frames.SetNum(InNumFrames);
	D3D12_HEAP_PROPERTIES heapProps {};
	heapProps.Type = D3D12_HEAP_TYPE_DEFAULT;
	heapProps.CPUPageProperty = D3D12_CPU_PAGE_PROPERTY_UNKNOWN;
	heapProps.MemoryPoolPreference = D3D12_MEMORY_POOL_UNKNOWN;
	heapProps.CreationNodeMask =
	heapProps.VisibleNodeMask = GetNodeMask();

	const uint32 Alignment = (uint32) InImageSizeAlignment;
	const uint32 AlignedWidth = Align(InMaxWidth, Alignment);
	const uint32 AlignedHeight = Align(InMaxHeight, Alignment);

	check(!CodecInfo.PixelFormats.IsEmpty());
	D3D12_RESOURCE_DESC desc {};
	desc.Dimension = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	desc.Alignment = 0;
	desc.Width = AlignedWidth;
	desc.Height = AlignedHeight;
	desc.DepthOrArraySize = 1;
	desc.MipLevels = 1;
	desc.Format = CodecInfo.PixelFormats[0];
	desc.SampleDesc.Count = 1;
	desc.SampleDesc.Quality = 0;
	desc.Layout = D3D12_TEXTURE_LAYOUT_UNKNOWN;
	desc.Flags = D3D12_RESOURCE_FLAG_NONE;

	HRESULT Result;
	for(int32 i=0; i<InNumFrames; ++i)
	{
		newdpb->Frames[i] = MakeShared<FDecodedFrame, ESPMode::ThreadSafe>();
		newdpb->Frames[i]->IndexInPictureBuffer = i;
		if ((Result = D3D12Device->CreateCommittedResource(&heapProps, D3D12_HEAP_FLAG_NONE, &desc, D3D12_RESOURCE_STATE_COMMON, nullptr, __uuidof(ID3D12Resource), (void**)newdpb->Frames[i]->Texture.GetInitReference())) != S_OK)
		{
			return PostError(Result, TEXT("CreateCommittedResource() failed while creating the DPB"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		}
		if ((Result = newdpb->Frames[i]->Sync.Create(D3D12Device, 0)) != S_OK)
		{
			return PostError(Result, TEXT("Creating fence failed while creating the DPB"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		}
	}
	// Add all frames to the available queue.
	for(auto& it : newdpb->Frames)
	{
		newdpb->AvailableQueue.Push(it);
	}
	OutDPB = MoveTemp(newdpb);
	CurrentConfig.MaxDecodedWidth = InMaxWidth;
	CurrentConfig.MaxDecodedHeight = InMaxHeight;
	return true;
}






bool FD3D12VideoDecoder::PrepareBitstreamBuffer(const TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe>& InFrameDecodeResourceToPrepare, uint32 InMaxInputBufferSize)
{
	check(InFrameDecodeResourceToPrepare);
	// Align to 256KiB multiples to reduce the number of times this needs to be marginally enlarged.
	InMaxInputBufferSize = Align(InMaxInputBufferSize, 1024*256);
	if (InFrameDecodeResourceToPrepare && (!InFrameDecodeResourceToPrepare->D3DBitstreamBuffer.IsValid() || InFrameDecodeResourceToPrepare->D3DBitstreamBufferAllocatedSize < InMaxInputBufferSize))
	{
		InFrameDecodeResourceToPrepare->D3DBitstreamBuffer.SafeRelease();
		InFrameDecodeResourceToPrepare->D3DBitstreamBufferAllocatedSize = 0;

		HRESULT Result;

		D3D12_HEAP_PROPERTIES HeapProps {};
		HeapProps.Type = D3D12_HEAP_TYPE_UPLOAD;
		HeapProps.CPUPageProperty = D3D12_CPU_PAGE_PROPERTY_UNKNOWN;
		HeapProps.MemoryPoolPreference = D3D12_MEMORY_POOL_UNKNOWN;
		HeapProps.CreationNodeMask =
		HeapProps.VisibleNodeMask = GetNodeMask();

		D3D12_RESOURCE_DESC desc {};
		desc.Dimension = D3D12_RESOURCE_DIMENSION_BUFFER,
		desc.Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
		desc.Width = InMaxInputBufferSize;
		desc.Height = 1;
		desc.DepthOrArraySize = 1;
		desc.MipLevels = 1;
		desc.Format = DXGI_FORMAT_UNKNOWN;
		desc.SampleDesc.Count = 1;
		desc.SampleDesc.Quality = 0;
		desc.Layout = D3D12_TEXTURE_LAYOUT_ROW_MAJOR;
		desc.Flags = D3D12_RESOURCE_FLAG_NONE;

		if ((Result = D3D12Device->CreateCommittedResource(&HeapProps, D3D12_HEAP_FLAG_NONE, &desc, D3D12_RESOURCE_STATE_GENERIC_READ, nullptr, __uuidof(ID3D12Resource), (void**)InFrameDecodeResourceToPrepare->D3DBitstreamBuffer.GetInitReference())) != S_OK)
		{
			return PostError(Result, TEXT("Bitstream buffer CreateCommittedResource() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		}
		InFrameDecodeResourceToPrepare->D3DBitstreamBufferAllocatedSize = InMaxInputBufferSize;
	}
	return true;
}


static TSharedPtr<FD3D12VideoDecoderFactory, ESPMode::ThreadSafe> Self;
} // namespace ElectraVideoDecodersD3D12Video


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/
void FD3D12VideoDecoder::Startup()
{
	if (!ElectraVideoDecodersD3D12Video::bDisableThisDecoder)
	{
		// Make sure the codec factory module has been loaded.
		FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

		TUniquePtr<ElectraVideoDecodersD3D12Video::FCodecFormatHelper> FormatHelper(new ElectraVideoDecodersD3D12Video::FCodecFormatHelper);
		// Not a single supported format?
		if (FormatHelper->FindSupportedFormats(nullptr) != 0)
		{
			// Create a factory with the current formats.
			ElectraVideoDecodersD3D12Video::Self = MakeShared<ElectraVideoDecodersD3D12Video::FD3D12VideoDecoderFactory, ESPMode::ThreadSafe>(MoveTemp(FormatHelper));
			// Register as modular feature.
			IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), ElectraVideoDecodersD3D12Video::Self.Get());
		}
		else
		{
			UE_LOGF(LogD3D12VideoDecodersElectra, Log, "D3D12 video decoding will not be used since no supported format was found.");
		}
	}
	else
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Log, "D3D12 video decoding will not be used since it is disabled.");
	}
}

void FD3D12VideoDecoder::Shutdown()
{
	if (ElectraVideoDecodersD3D12Video::Self.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), ElectraVideoDecodersD3D12Video::Self.Get());
		ElectraVideoDecodersD3D12Video::Self.Reset();
	}
}
