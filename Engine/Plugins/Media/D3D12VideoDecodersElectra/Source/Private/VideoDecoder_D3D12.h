// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FD3D12VideoDecoder
{
public:
	static void Startup();
	static void Shutdown();
};
