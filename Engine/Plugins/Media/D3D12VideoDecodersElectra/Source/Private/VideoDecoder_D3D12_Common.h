// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Misc/TVariant.h"
#include "Containers/Queue.h"

#include "IElectraDecoder.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputVideo.h"

#include "ElectraDecodersUtils.h"

/*********************************************************************************************************************/
#include COMPILED_PLATFORM_HEADER(PlatformHeaders_Video_D3D.h)
#include "DecoderErrors_D3D12.h"
/*********************************************************************************************************************/


namespace ElectraVideoDecodersD3D12Video
{

class FCodecFormatHelper
{
public:
	enum class ECodecType
	{
		H264,
		H265,
		VP9
	};

	struct FCodecInfo
	{
		ECodecType CodecType;
		bool b10Bit = false;
		GUID ProfileGUID;
		TArray<DXGI_FORMAT> PixelFormats;
		bool operator == (const FCodecInfo& rhs) const
		{
			return CodecType == rhs.CodecType && b10Bit == rhs.b10Bit && ProfileGUID == rhs.ProfileGUID && PixelFormats == rhs.PixelFormats;
		}
	};

	FCodecFormatHelper() = default;
	~FCodecFormatHelper() = default;
	int32 FindSupportedFormats(ID3D12Device* InD3D12Device);
	const FCodecInfo* HaveFormat(ECodecType InType, int32 InNumBits);

	TRefCountPtr<ID3D12VideoDevice> GetVideoDevice()
	{
		return DxVideoDevice;
	}
	uint32 GetVideoDeviceNodeIndex()
	{
		return DxDeviceNodeIndex;
	}

private:
	// The profile GUIDs we support.
	const GUID D3D12_VIDEO_DECODE_PROFILE_H264                   = { 0x1b81be68, 0xa0c7, 0x11d3, { 0xb9, 0x84, 0x00, 0xc0, 0x4f, 0x2e, 0x73, 0xc5 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_HEVC_MAIN              = { 0x5b11d51b, 0x2f4c, 0x4452, { 0xbc, 0xc3, 0x09, 0xf2, 0xa1, 0x16, 0x0c, 0xc0 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_HEVC_MAIN10            = { 0x107af0e0, 0xef1a, 0x4d19, { 0xab, 0xa8, 0x67, 0xa1, 0x63, 0x07, 0x3d, 0x13 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_VP9                    = { 0x463707f8, 0xa1d0, 0x4585, { 0x87, 0x6d, 0x83, 0xaa, 0x6d, 0x60, 0xb8, 0x9e } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_VP9_10BIT_PROFILE2     = { 0xa4c749ef, 0x6ecf, 0x48aa, { 0x84, 0x48, 0x50, 0xa7, 0xa1, 0x16, 0x5f, 0xf7 } };
	/*
	const GUID D3D12_VIDEO_DECODE_PROFILE_VP8                    = { 0x90b899ea, 0x3a62, 0x4705, { 0x88, 0xb3, 0x8d, 0xf0, 0x4b, 0x27, 0x44, 0xe7 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_AV1_PROFILE0           = { 0xb8be4ccb, 0xcf53, 0x46ba, { 0x8d, 0x59, 0xd6, 0xb8, 0xa6, 0xda, 0x5d, 0x2a } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_AV1_PROFILE1           = { 0x6936ff0f, 0x45b1, 0x4163, { 0x9c, 0xc1, 0x64, 0x6e, 0xf6, 0x94, 0x61, 0x08 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_AV1_PROFILE2           = { 0x0c5f2aa1, 0xe541, 0x4089, { 0xbb, 0x7b, 0x98, 0x11, 0x0a, 0x19, 0xd7, 0xc8 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_AV1_12BIT_PROFILE2     = { 0x17127009, 0xa00f, 0x4ce1, { 0x99, 0x4e, 0xbf, 0x40, 0x81, 0xf6, 0xf3, 0xf0 } };
	const GUID D3D12_VIDEO_DECODE_PROFILE_AV1_12BIT_PROFILE2_420 = { 0x2d80bed6, 0x9cac, 0x4835, { 0x9e, 0x91, 0x32, 0x7b, 0xbc, 0x4f, 0x9e, 0xe8 } };
	*/

	TArray<FCodecInfo> CodecInfos;
	TRefCountPtr<ID3D12VideoDevice> DxVideoDevice;
	uint32 DxDeviceNodeIndex = 0;
};

class FSyncObject
{
public:
	virtual ~FSyncObject()
	{
		AwaitCompletion(0);
		if (EventHandle != INVALID_HANDLE_VALUE)
		{
			CloseHandle(EventHandle);
		}
		Fence.SafeRelease();
	}
	HRESULT Create(const TRefCountPtr<ID3D12Device>& InDevice, uint64 InInitialValue)
	{
		Value = InInitialValue;
		HRESULT Result = InDevice->CreateFence(Value, D3D12_FENCE_FLAG_NONE, __uuidof(ID3D12Fence), (void**)Fence.GetInitReference());
		if (Result == S_OK)
		{
			EventHandle = CreateEvent(nullptr, false, false, nullptr);
			check(EventHandle != INVALID_HANDLE_VALUE);
			Result = EventHandle != INVALID_HANDLE_VALUE ? Result : ERROR_INVALID_HANDLE;
		}
		return Result;
	}

	TRefCountPtr<ID3D12Fence> GetFence()
	{
		return Fence;
	}
	void* GetID3D12Fence()
	{
		return Fence.GetReference();
	}
	uint64 IncrementAndGetNewFenceValue()
	{
		++Value;
		return Value;
	}
	uint64& GetFenceValue()
	{
		return Value;
	}

	bool AwaitCompletion(uint32 InTimeoutMillisec)
	{
		if (Fence.IsValid())
		{
			uint64 CompletedValue = Fence->GetCompletedValue();
			if (CompletedValue < Value)
			{
				HRESULT Result = Fence->SetEventOnCompletion(Value, EventHandle);
				if (!ensure(Result == S_OK))
				{
					return false;
				}
				return WaitForSingleObjectEx(EventHandle, InTimeoutMillisec, false) == WAIT_OBJECT_0;
			}
			return true;
		}
		return false;
	}
private:
	TRefCountPtr<ID3D12Fence> Fence;
	HANDLE EventHandle = INVALID_HANDLE_VALUE;
	uint64 Value = 0;
};

struct FDecodedFrame
{
	TRefCountPtr<ID3D12Resource> Texture;
	FSyncObject Sync;
	int32 IndexInPictureBuffer = 0;
};

class FDecodedPictureBuffer
{
public:
	~FDecodedPictureBuffer();
	void ReleaseAllFrames(int32 InWaitForEachFrameMillis);
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> GetFrameAtIndex(int32 InIndex);
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> GetFrameForResource(const ID3D12Resource* InResource);
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> GetNextUnusedFrame();
	void ReturnUnusedFrameToAvailableQueue(TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>&& InFrame);
	void ReturnFrameToAvailableQueue(TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>&& InFrame);
public:
	TArray<TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>> Frames;
	TArray<TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>> AvailableQueue;
};

class FVideoDecoderOutputD3D12Electra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputD3D12Electra() = default;

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	EOutputType GetOutputType() const
	{
		return OutputType;
	}

	int32 GetWidth() const override
	{
		return ImageWidth;
	}
	int32 GetHeight() const override
	{
		return ImageHeight;
	}
	int32 GetDecodedWidth() const override
	{
		return Width;
	}
	int32 GetDecodedHeight() const override
	{
		return Height;
	}
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{
		return Crop;
	}
	int32 GetAspectRatioW() const override
	{
		return AspectW;
	}
	int32 GetAspectRatioH() const override
	{
		return AspectH;
	}
	int32 GetFrameRateNumerator() const override
	{
		return FrameRateN;
	}
	int32 GetFrameRateDenominator() const override
	{
		return FrameRateD;
	}
	int32 GetNumberOfBits() const override
	{
		return NumBits;
	}
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{
		OutExtraValues = ExtraValues;
	}
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputD3D12Electra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{
		return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported;
	}

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return 1;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		check(InBufferIndex == 0);
		// No CPU data here.
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
		check(InBufferIndex == 0);
		if (DecodedFrame.IsValid())
		{
			return DecodedFrame->Texture;
		}
		return nullptr;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		return BufferFormat;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		return BufferEncoding;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		return Pitch;
	}
	bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
		if (InBufferIndex == 0 && DecodedFrame.IsValid())
		{
			// Provide the caller with the decode fence and associated value.
			SyncObject.Sync = DecodedFrame->Sync.GetFence();
			SyncObject.SyncValue = DecodedFrame->Sync.GetFenceValue();
			// Now, since we are asked to provide the sync object we *CONTRACTUALLY* assume that the caller will *DO*
			// something with the output.
			// As such, we also return the fence as the copy-complete fence with an increased fence value.
			SyncObject.CopyDoneSync = DecodedFrame->Sync.GetFence();
			SyncObject.CopyDoneSyncValue = DecodedFrame->Sync.IncrementAndGetNewFenceValue();
			return true;
		}
		return false;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 ImageWidth = 0;
	int32 ImageHeight = 0;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;
	EOutputType OutputType = EOutputType::Output;

	uint64 UserValue0 = 0;
	bool bDoNotOutput = false;

	uint32 Codec4CC = 0;
	EPixelFormat BufferFormat = EPixelFormat::PF_Unknown;
	EElectraTextureSamplePixelEncoding BufferEncoding = EElectraTextureSamplePixelEncoding::Native;


	TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe> OwningDPB;
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> DecodedFrame;
};


class FD3D12VideoDecoder : public IElectraDecoder
{
public:
	static bool D3D12VIDEODECODERSELECTRA_API CheckPlatformDecodeCapabilities(D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InOutDecodeSupport,  const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InOptions);
	FD3D12VideoDecoder(const FCodecFormatHelper::FCodecInfo& InCodecInfo, const D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InDecodeSupport, const TMap<FString, FVariant>& InOptions, const TRefCountPtr<ID3D12Device>& InD3D12Device, const TRefCountPtr<ID3D12VideoDevice>& InVideoDevice, uint32 InVideoDeviceNodeIndex);
	virtual ~FD3D12VideoDecoder();
protected:
	EType GetType() const override
	{ return IElectraDecoder::EType::Video; }
	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;
	FError GetError() const override
	{ return LastError; }
	void Close() override;
	ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) = 0;
	bool ResetToCleanStart() override;
	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) = 0;
	EDecoderError SendEndOfData() = 0;
	EDecoderError Flush() = 0;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;
	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() = 0;
	void Suspend() override
	{ }
	void Resume() override
	{ }

protected:
	bool PostError(HRESULT ApiReturnValue, FString Message, int32 Code)
	{
		LastError.Code = Code;
		LastError.SdkCode = ApiReturnValue;
		LastError.Message = MoveTemp(Message);
		return false;
	}

	class FAutoReturnUnusedFrame
	{
		public:
			FAutoReturnUnusedFrame(const TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe>& InDPB, const TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe>& InFrame)
				: OwningDPB(InDPB), ThisFrame(InFrame)
			{}
			~FAutoReturnUnusedFrame()
			{
				if (ThisFrame.IsValid())
				{
					OwningDPB->ReturnUnusedFrameToAvailableQueue(MoveTemp(ThisFrame));
				}
			}
			void ReleaseOwnership()
			{
				ThisFrame.Reset();
			}
		private:
			TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe> OwningDPB;
			TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> ThisFrame;
	};

	// A structure holding every resource needed to decode one frame.
	struct FFrameDecodeResource
	{
		// Decoder keep-alive resources.
		TRefCountPtr<ID3D12VideoDecoder> D3DDecoder;
		TRefCountPtr<ID3D12VideoDecoderHeap> D3DDecoderHeap;

		// Active decoding resource.
		enum
		{
			kMaxRefFrames = 32
		};
		ID3D12Resource* ReferenceFrameList[kMaxRefFrames] {};
		UINT ReferenceFrameListSubRes[kMaxRefFrames] { 0 };
		TRefCountPtr<ID3D12Resource> D3DBitstreamBuffer;
		uint32 D3DBitstreamBufferAllocatedSize = 0;
		uint32 D3DBitstreamBufferPayloadSize = 0;

		struct FInputEmpty
		{
		};

		struct FInputH264
		{
			DXVA_PicParams_H264 PicParams;
			DXVA_Qmatrix_H264 QuantMtx;
			TArray<DXVA_Slice_H264_Short> SliceHeaders;
		};

		struct FInputH265
		{
			DXVA_PicParams_HEVC PicParams;
			DXVA_Qmatrix_HEVC QuantMtx;
			TArray<DXVA_Slice_HEVC_Short> SliceHeaders;
		};

		TVariant<FInputEmpty, FInputH264, FInputH265> PicInput;
	};

	struct FDecoderConfiguration
	{
		void Reset()
		{
			MaxDecodedWidth = 0;
			MaxDecodedHeight = 0;
			MaxNumInDPB = 0;
			VideoDecoderHeap.SafeRelease();
			VideoDecoderDPBWidth = 0;
			VideoDecoderDPBHeight = 0;
		}
		int32 MaxDecodedWidth = 0;
		int32 MaxDecodedHeight = 0;
		int32 MaxNumInDPB = 0;
		TRefCountPtr<ID3D12VideoDecoderHeap> VideoDecoderHeap;
		int32 VideoDecoderDPBWidth = 0;
		int32 VideoDecoderDPBHeight = 0;
	};

	bool InternalDecoderCreate();
	void ReturnAllFrames();

	bool CreateDecoderHeap(int32 InDPBSize, int32 InMaxWidth, int32 InMaxHeight, int32 InImageSizeAlignment);
	bool CreateDPB(TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe>& OutDPB, int32 InMaxWidth, int32 InMaxHeight, int32 InImageSizeAlignment, int32 InNumFrames);

	constexpr uint32 GetNodeMask() const
	{ return VideoDeviceNodeIndex; }

	bool PrepareBitstreamBuffer(const TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe>& InFrameDecodeResourceToPrepare, uint32 InMaxInputBufferSize);

	EDecoderError ExecuteCommonDecode(const D3D12_VIDEO_DECODE_INPUT_STREAM_ARGUMENTS& InInputArgs, const D3D12_VIDEO_DECODE_OUTPUT_STREAM_ARGUMENTS& InOutputArgs);

	virtual bool InternalResetToCleanStart() = 0;

	FCodecFormatHelper::FCodecInfo CodecInfo;
	D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT DecodeSupport;
	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;
	TRefCountPtr<ID3D12Device> D3D12Device;
	TRefCountPtr<ID3D12VideoDevice> VideoDevice;
	uint32 VideoDeviceNodeIndex = 0;
	IElectraDecoder::FError LastError;

	TUniquePtr<FSyncObject> VideoDecoderSync;
	TRefCountPtr<ID3D12CommandQueue> VideoDecoderCommandQueue;
	TRefCountPtr<ID3D12CommandAllocator> VideoDecoderCommandAllocator;
	TRefCountPtr<ID3D12VideoDecodeCommandList> VideoDecoderCommandList;
	TRefCountPtr<ID3D12VideoDecoder> VideoDecoder;
	uint32 StatusReportFeedbackNumber = 0;

	FDecoderConfiguration CurrentConfig;

	// Queue of frame decode resources that are available again for re-use.
	TQueue<TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe>> AvailableFrameDecodeResourceQueue;
	// Currently active frame decode resources that await completion.
	//TArray<TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe>> ActiveFrameDecodeResources;

	TSharedPtr<FDecodedPictureBuffer, ESPMode::ThreadSafe> DPB;
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> MissingReferenceFrame;

	uint32 RunningFrameNumLo = 0;
	uint32 RunningFrameNumHi = 0;
	bool bIsDraining = false;

	TArray<TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe>> FramesInDecoder;
	TArray<TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe>> FramesReadyForOutput;
	TArray<TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe>> FramesGivenOutForOutput;
};

}
