// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoder_D3D12_H264.h"
#include "D3D12VideoDecodersElectraModule.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H264.h"
#include "PixelFormat.h"


namespace ElectraVideoDecodersD3D12Video
{

FD3D12VideoDecoder_H264::FD3D12VideoDecoder_H264(const FCodecFormatHelper::FCodecInfo& InCodecInfo, const D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InDecodeSupport, const TMap<FString, FVariant>& InOptions, const TRefCountPtr<ID3D12Device>& InD3D12Device, const TRefCountPtr<ID3D12VideoDevice>& InVideoDevice, uint32 InVideoDeviceNodeIndex)
	: FD3D12VideoDecoder(InCodecInfo, InDecodeSupport, InOptions, InD3D12Device, InVideoDevice, InVideoDeviceNodeIndex)
{
}

FD3D12VideoDecoder_H264::~FD3D12VideoDecoder_H264()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

IElectraDecoder::ECSDCompatibility FD3D12VideoDecoder_H264::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// No decoder yet means we are compatible.
	if (!VideoDecoder.IsValid())
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	FBitstreamParamsH264 temp;
	if (GetCodecSpecificDataH264(temp, CSDAndAdditionalOptions, false) == IElectraDecoder::EDecoderError::Error)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	// We can only check against a single provided SPS. If none or several, start over.
	if (temp.SPSs.Num() != 1)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}

#if 0
	const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet sps = temp.SPSs.CreateConstIterator().Value();
	int32 NewWidth, NewHeight, NewDPBSize;
	NewDPBSize = sps.GetDPBSize();
	sps.GetDisplaySize(NewWidth, NewHeight);
	if (NewDPBSize > CurrentConfig.MaxNumInDPB ||
		NewWidth > CurrentConfig.MaxDecodedWidth ||
		NewHeight > CurrentConfig.MaxDecodedHeight)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	return IElectraDecoder::ECSDCompatibility::Compatible;
#else
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
#endif
}

TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> FD3D12VideoDecoder_H264::CreateBitstreamProcessor()
{
	TMap<FString, FVariant> DecoderFeatures;
	GetFeatures(DecoderFeatures);
	return FElectraDecoderBitstreamProcessorH264::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
}


IElectraDecoder::EDecoderError FD3D12VideoDecoder_H264::GetCodecSpecificDataH264(FBitstreamParamsH264& OutBitstreamParamsH264, const TMap<FString, FVariant>& InAdditionalOptions, bool bIsRequired)
{
	TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("csd"));
	// Split the CSD into individual NAL units.
	TArray<ElectraDecodersUtil::MPEG::H264::FNaluInfo> NalUnits;
	if (!ElectraDecodersUtil::MPEG::H264::ParseBitstreamForNALUs(NalUnits, CSD.GetData(), CSD.Num()))
	{
		if (bIsRequired)
		{
			PostError(0, TEXT("Failed to locate the NALUs in the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
			return IElectraDecoder::EDecoderError::Error;
		}
		else
		{
			return IElectraDecoder::EDecoderError::NoBuffer;
		}
	}
	// Parse the SPS and PPS
	for(int32 i=0; i<NalUnits.Num(); ++i)
	{
		if (NalUnits[i].Type == 7)
		{
			if (!ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(OutBitstreamParamsH264.SPSs, CSD.GetData() + NalUnits[i].Offset + NalUnits[i].UnitLength, NalUnits[i].Size))
			{
				PostError(0, TEXT("Failed to parse the SPS from the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else if (NalUnits[i].Type == 8)
		{
			if (!ElectraDecodersUtil::MPEG::H264::ParsePictureParameterSet(OutBitstreamParamsH264.PPSs, OutBitstreamParamsH264.SPSs, CSD.GetData() + NalUnits[i].Offset + NalUnits[i].UnitLength, NalUnits[i].Size))
			{
				PostError(0, TEXT("Failed to parse the PPS from the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else if (NalUnits[i].Type == 14 || NalUnits[i].Type == 20 || NalUnits[i].Type == 21)
		{
			PostError(0, TEXT("Unsupported SVC, MVC or AVC3D extension"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}



IElectraDecoder::EDecoderError FD3D12VideoDecoder_H264::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (!InInputAccessUnit.Data || !InInputAccessUnit.DataSize)
	{
		return IElectraDecoder::EDecoderError::None;
	}

	bool bGotCSD = true;
	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
	{
		bGotCSD = GetCodecSpecificDataH264(BitstreamParamsH264, InAdditionalOptions, false) == IElectraDecoder::EDecoderError::None;
	}

	// We need to isolate the slices that make up this frame.
	TArray<FBitstreamParamsH264::FSliceDecodeInfo> SliceInfos;
	bool bIsIDR = false;
	// Go over each of the NALUs in the bitstream.
	const uint8* Start = (const uint8*)InInputAccessUnit.Data;
	const uint8* End = Start + InInputAccessUnit.DataSize;
	bool bGotSPS = false;
	bool bGotPPS = false;
	while(Start < End)
	{
		uint32 NaluLength = ((uint32)Start[0] << 24) | ((uint32)Start[1] << 16) | ((uint32)Start[2] << 8) | ((uint32)Start[3]);
		uint32 nut = Start[4] & 0x1f;
		uint32 refIdc = (Start[4] >> 5) & 3;
		if (nut == 1 || nut == 5)
		{
			if (nut == 5)
			{
				bIsIDR = true;
			}

			bGotCSD |= (bGotSPS && bGotPPS);
			if (!bGotCSD)
			{
				PostError(0, TEXT("No SPS and PPS found in CSD or inband, cannot decode slice"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}

			FBitstreamParamsH264::FSliceDecodeInfo& SliceInfo = SliceInfos.Emplace_GetRef();
			SliceInfo.NalUnitType = (uint8)nut;
			SliceInfo.NalRefIdc = (uint8)refIdc;
			ElectraDecodersUtil::MPEG::H264::FBitstreamReaderH264 br;
			TUniquePtr<ElectraDecodersUtil::MPEG::H264::FRBSP> SliceRBSP;
			if (!ElectraDecodersUtil::MPEG::H264::ParseSliceHeader(SliceRBSP, br, SliceInfo.Header, BitstreamParamsH264.SPSs, BitstreamParamsH264.PPSs, Start+4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream slice header"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}

			// Check that the PPS is the same for all slices
			if (SliceInfos.Num() > 1 && SliceInfos.Last().Header.pic_parameter_set_id != SliceInfos[0].Header.pic_parameter_set_id)
			{
				PostError(0, TEXT("Picture parameter set id differs across frame slices!"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}

			// Fill in the remaining slice information
			SliceInfo.NalUnitStartAddress = Start + 4;
			SliceInfo.NumBytesInSlice = NaluLength;
		}
		else if (nut == 2 || nut == 3 || nut == 4)
		{
			PostError(0, TEXT("Found partitioned slice data that should not appear in the supported profiles"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		// Inband SPS
		else if (nut == 7)
		{
			if (!ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(BitstreamParamsH264.SPSs, Start + 4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream inband SPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
			bGotSPS = true;
		}
		// Inband PPS
		else if (nut == 8)
		{
			if (!ElectraDecodersUtil::MPEG::H264::ParsePictureParameterSet(BitstreamParamsH264.PPSs, BitstreamParamsH264.SPSs, Start + 4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream inband PPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
			bGotPPS = true;
		}
		// SVC / AVC 3D extension?
		else if (nut == 14 || nut == 20 || nut == 21)
		{
			PostError(0, TEXT("Unsupported SVC, MVC or AVC3D extension"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}

		Start += NaluLength + 4;
	}

	// Any slices to decode?
	if (SliceInfos.Num())
	{
		// Create a new decoder if we do not have one. This does not require any information about the resolution or DPB.
		if (!VideoDecoder.IsValid())
		{
			if (!InternalDecoderCreate())
			{
				return IElectraDecoder::EDecoderError::Error;
			}
		}

		const ElectraDecodersUtil::MPEG::H264::FPictureParameterSet* ppsPtr = BitstreamParamsH264.PPSs.Find(SliceInfos[0].Header.pic_parameter_set_id);
		if (!ppsPtr)
		{
			PostError(0, TEXT("Reference picture parameter set not found"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet* spsPtr = BitstreamParamsH264.SPSs.Find(ppsPtr->seq_parameter_set_id);
		if (!spsPtr)
		{
			PostError(0, TEXT("Reference sequence parameter set not found"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		// On an IDR frame check if we need a new decoder, either because we have none or the relevant
		// decoding parameters changed.
		if (bIsIDR)
		{
			const int32 Alignment = 16;
			const int32 DPBSize = spsPtr->GetDPBSize();
			int32 dw, dh;
			spsPtr->GetDisplaySize(dw, dh);

			// Check if the decoder heap parameters have changed such that we have to create a new one.
			if (DPBSize != CurrentConfig.MaxNumInDPB || dw != CurrentConfig.VideoDecoderDPBWidth || dh != CurrentConfig.VideoDecoderDPBHeight)
			{
				CurrentConfig.VideoDecoderHeap.SafeRelease();
			}
			if (!CurrentConfig.VideoDecoderHeap.IsValid())
			{
				if (!CreateDecoderHeap(DPBSize, dw, dh, Alignment))
				{
					return IElectraDecoder::EDecoderError::Error;
				}
			}

			if (!DPB.IsValid())
			{
				// As far as the decoded frames go, their size can be the maximum that is required
				// for this stream (the largest resolution).
				const int32 Width = (int32) DecodeSupport.Width;
				const int32 Height = (int32) DecodeSupport.Height;
				const int32 NumFrames = spsPtr->GetDPBSize() + 2;	// 1 extra for the current frame that's not in the DPB yet, and 1 extra that acts as a 'missing' frame.
				if (!CreateDPB(DPB, Width, Height, Alignment, NumFrames))
				{
					return IElectraDecoder::EDecoderError::Error;
				}
				MissingReferenceFrame = DPB->GetNextUnusedFrame();
				if (!MissingReferenceFrame.IsValid())
				{
					PostError(0, TEXT("Could not create empty frame used to fill in for missing frames"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
					return IElectraDecoder::EDecoderError::Error;
				}
			}
		}
		IElectraDecoder::EDecoderError Error = DecodeSlicesH264(InInputAccessUnit, SliceInfos, *spsPtr, *ppsPtr);
		return Error;
	}

	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H264::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bIsDraining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bIsDraining = true;
	TArray<ElectraDecodersUtil::MPEG::H264::FOutputFrameInfo> OutputFrameInfos, UnrefFrameInfos;
	BitstreamParamsH264.DPBPOC.Flush(OutputFrameInfos, UnrefFrameInfos);
	return HandleOutputListH264(OutputFrameInfos);
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H264::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Wait for a while for the most recent decode operation to have finished.
	if (VideoDecoderSync.IsValid())
	{
		VideoDecoderSync->AwaitCompletion(500);
	}
	BitstreamParamsH264.Reset();
	ReturnAllFrames();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H264::DecodeSlicesH264(const FInputAccessUnit& InInputAccessUnit, const TArray<FBitstreamParamsH264::FSliceDecodeInfo>& InSliceInfos, const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet& InSequenceParameterSet, const ElectraDecodersUtil::MPEG::H264::FPictureParameterSet& InPictureParameterSet)
{
	// The caller needs to make sure we do not get called without slices
	check(InSliceInfos.Num());

	if (!VideoDecoderSync.IsValid())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// The previous operation must have completed, primarily because we (may) need the decoded frame from before
	// as a reference frame for this call and that frame thus needs to have finished.
	if (!VideoDecoderSync->AwaitCompletion(500))
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Warning, "DecodeSlicesH264() waited too long for the previous operation to complete. Trying again later.");
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Some capability checks
	if (InSequenceParameterSet.mb_adaptive_frame_field_flag || InSliceInfos[0].Header.field_pic_flag || InSliceInfos[0].Header.bottom_field_flag)
	{
		PostError(0, FString::Printf(TEXT("DecodeSlicesH264() failed. Cannot decode interlaced video.")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}
	if (InPictureParameterSet.num_slice_groups_minus1)
	{
		PostError(0, FString::Printf(TEXT("DecodeSlicesH264() failed. Slice groups are not supported.")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}



	check(DPB.IsValid());
	if (!DPB.IsValid())
	{
		PostError(0, FString::Printf(TEXT("DecodeSlicesH264() failed. There is no DPB")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}

	// Get the frames that are currently referenced by the DPB.
	TArray<ElectraDecodersUtil::MPEG::H264::FSlicePOCVars::FReferenceFrameListEntry> RefFrames;
	BitstreamParamsH264.DPBPOC.GetCurrentReferenceFrames(RefFrames);

	// Go over all the frames that we have already handed out for display.
	// These should have been copied or converted the moment we handed them out and are thus
	// available for use again, provided the DPB does not still need them for reference.
	for(int32 i=0; i<FramesGivenOutForOutput.Num(); ++i)
	{
		// FIXME: If we have to create a new DPB then the frames we handed out may be from the old DPB
		check(FramesGivenOutForOutput[i]->OwningDPB == DPB);
		bool bIsUnref = true;
		for(int32 j=0; j<RefFrames.Num(); ++j)
		{
			if (FramesGivenOutForOutput[i]->UserValue0 == RefFrames[j].UserFrameInfo.UserValue0)
			{
				bIsUnref = false;
				break;
			}
		}
		if (bIsUnref)
		{
			FramesGivenOutForOutput[i]->OwningDPB->ReturnFrameToAvailableQueue(MoveTemp(FramesGivenOutForOutput[i]->DecodedFrame));
			FramesGivenOutForOutput.RemoveAt(i);
			--i;
		}
	}

	// Get a target frame to decode into.
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> TargetFrame = DPB->GetNextUnusedFrame();
	if (!TargetFrame.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
	FAutoReturnUnusedFrame AutoRelease(DPB, TargetFrame);

	// Get an available frame decode resource
	TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe> fdr;
	if (!AvailableFrameDecodeResourceQueue.Dequeue(fdr))
	{
		fdr = MakeShared<FFrameDecodeResource, ESPMode::ThreadSafe>();
	}
	// Make sure the input is set to the correct type.
	if (fdr->PicInput.GetIndex() != fdr->PicInput.IndexOfType<FFrameDecodeResource::FInputH264>())
	{
		fdr->PicInput.Emplace<FFrameDecodeResource::FInputH264>(FFrameDecodeResource::FInputH264());
	}
	// Calculate the total input bitstream size.
	uint32 TotalSliceSize = 0;
	for(auto& si : InSliceInfos)
	{
		TotalSliceSize += si.NumBytesInSlice + 3;	// need to prepend each slice with a 0x000001 startcode
	}
	// If necessary reallocate the bitstream buffer.
	if (!PrepareBitstreamBuffer(fdr, TotalSliceSize))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	FFrameDecodeResource::FInputH264& input = fdr->PicInput.Get<FFrameDecodeResource::FInputH264>();
	input.SliceHeaders.SetNumUninitialized(InSliceInfos.Num());

	// Copy the slices into the buffer and set up the short slice headers
	HRESULT Result;
	uint8* BufferPtr = nullptr;
	if ((Result = fdr->D3DBitstreamBuffer->Map(0, nullptr, (void**)&BufferPtr)) != S_OK)
	{
		PostError(0, TEXT("ID3D12Resource::Map() failed for bitstream buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}
	uint8* BufferBase = BufferPtr;
	for(int32 ns=0; ns<InSliceInfos.Num(); ++ns)
	{
		input.SliceHeaders[ns].BSNALunitDataLocation = BufferPtr - BufferBase;
		input.SliceHeaders[ns].SliceBytesInBuffer = 3 + InSliceInfos[ns].NumBytesInSlice;
		input.SliceHeaders[ns].wBadSliceChopping = 0;
		*BufferPtr++ = 0;
		*BufferPtr++ = 0;
		*BufferPtr++ = 1;
		FMemory::Memcpy(BufferPtr, InSliceInfos[ns].NalUnitStartAddress, InSliceInfos[ns].NumBytesInSlice);
		BufferPtr += InSliceInfos[ns].NumBytesInSlice;
	}
	check(BufferPtr - BufferBase == TotalSliceSize);
	fdr->D3DBitstreamBuffer->Unmap(0, nullptr);
	fdr->D3DBitstreamBufferPayloadSize = BufferPtr - BufferBase;


	D3D12_VIDEO_DECODE_OUTPUT_STREAM_ARGUMENTS osa {};
	osa.pOutputTexture2D = TargetFrame->Texture;
    osa.OutputSubresource = 0;
	osa.ConversionArguments.Enable = false;

	// Input picture parameters
	DXVA_PicParams_H264& pp = input.PicParams;
	FMemory::Memzero(pp);

	pp.wFrameWidthInMbsMinus1 = (USHORT) InSequenceParameterSet.pic_width_in_mbs_minus1;
	pp.wFrameHeightInMbsMinus1 = (USHORT) InSequenceParameterSet.pic_height_in_map_units_minus1;
	pp.num_ref_frames = (UCHAR) InSequenceParameterSet.max_num_ref_frames;
	pp.residual_colour_transform_flag = InSequenceParameterSet.separate_colour_plane_flag;
	pp.chroma_format_idc = InSequenceParameterSet.chroma_format_idc;
	pp.RefPicFlag = InSliceInfos[0].NalRefIdc != 0 ? 1 : 0;
	pp.constrained_intra_pred_flag = InPictureParameterSet.constrained_intra_pred_flag;
	pp.weighted_pred_flag = InPictureParameterSet.weighted_pred_flag;
	pp.weighted_bipred_idc = InPictureParameterSet.weighted_bipred_idc;
	pp.MbsConsecutiveFlag = 1;
	pp.frame_mbs_only_flag = InSequenceParameterSet.frame_mbs_only_flag;
	pp.transform_8x8_mode_flag = InPictureParameterSet.transform_8x8_mode_flag;
	pp.MinLumaBipredSize8x8Flag = InSequenceParameterSet.profile_idc >= 77 && InSequenceParameterSet.level_idc >= 31;
	pp.IntraPicFlag = InSliceInfos[0].Header.slice_type % 5 == 2;	// or %5==4 if SI slices were allowed
	pp.bit_depth_luma_minus8 = (UCHAR) InSequenceParameterSet.bit_depth_luma_minus8;
	pp.bit_depth_chroma_minus8 = (UCHAR) InSequenceParameterSet.bit_depth_chroma_minus8;
	pp.Reserved16Bits = 3;
	if (++StatusReportFeedbackNumber == 0)
	{
		StatusReportFeedbackNumber = 1;
	}
	pp.StatusReportFeedbackNumber = StatusReportFeedbackNumber;
	pp.pic_init_qs_minus26 = (CHAR) InPictureParameterSet.pic_init_qs_minus26;
	pp.chroma_qp_index_offset = (CHAR) InPictureParameterSet.chroma_qp_index_offset;
	pp.second_chroma_qp_index_offset = (CHAR) InPictureParameterSet.second_chroma_qp_index_offset;

	// Since we have the accelerator parse the slice data and macroblocks we have to fill in the remaining structure members.
	pp.ContinuationFlag = 1;
	pp.pic_init_qp_minus26 = (CHAR) InPictureParameterSet.pic_init_qp_minus26;
	pp.num_ref_idx_l0_active_minus1 = (UCHAR) InPictureParameterSet.num_ref_idx_l0_default_active_minus1;
	pp.num_ref_idx_l1_active_minus1 = (UCHAR) InPictureParameterSet.num_ref_idx_l1_default_active_minus1;
	pp.log2_max_frame_num_minus4 = (UCHAR) InSequenceParameterSet.log2_max_frame_num_minus4;
	pp.pic_order_cnt_type = (UCHAR) InSequenceParameterSet.pic_order_cnt_type;
	pp.log2_max_pic_order_cnt_lsb_minus4 = (UCHAR) InSequenceParameterSet.log2_max_pic_order_cnt_lsb_minus4;
	pp.delta_pic_order_always_zero_flag = (UCHAR) InSequenceParameterSet.delta_pic_order_always_zero_flag;
	pp.direct_8x8_inference_flag = (UCHAR) InSequenceParameterSet.direct_8x8_inference_flag;
	pp.entropy_coding_mode_flag = (UCHAR) InPictureParameterSet.entropy_coding_mode_flag;
	pp.pic_order_present_flag = (UCHAR) InPictureParameterSet.bottom_field_pic_order_in_frame_present_flag;
	pp.num_slice_groups_minus1 = (UCHAR) InPictureParameterSet.num_slice_groups_minus1;
	pp.slice_group_map_type = (UCHAR) InPictureParameterSet.slice_group_map_type;
	pp.deblocking_filter_control_present_flag = (UCHAR) InPictureParameterSet.deblocking_filter_control_present_flag;
	pp.redundant_pic_cnt_present_flag = (UCHAR) InPictureParameterSet.redundant_pic_cnt_present_flag;
	check(InPictureParameterSet.slice_group_change_rate_minus1 == 0);

	// Start POC processing for this frame (first slice only)
	if (!BitstreamParamsH264.DPBPOC.BeginFrame(InSliceInfos[0].NalUnitType, InSliceInfos[0].NalRefIdc, InSliceInfos[0].Header, InSequenceParameterSet, InPictureParameterSet))
	{
		PostError(0, FString::Printf(TEXT("DecodeSlicesH264() failed. %s"), *BitstreamParamsH264.DPBPOC.GetLastError()), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}
	TArray<ElectraDecodersUtil::MPEG::H264::FOutputFrameInfo> OutputFrameInfos, UnrefFrameInfos;
	// Handle potentially missing frames. If there are any, an entry must be made in the DPB which could result in
	// output of one or many already decoded frames that we need to handle first.
	BitstreamParamsH264.DPBPOC.HandleMissingFrames(OutputFrameInfos, UnrefFrameInfos, InSliceInfos[0].NalUnitType, InSliceInfos[0].NalRefIdc, InSliceInfos[0].Header, InSequenceParameterSet);
	IElectraDecoder::EDecoderError MissingFrameOutputResult = HandleOutputListH264(OutputFrameInfos);
	OutputFrameInfos.Empty();
	UnrefFrameInfos.Empty();
	if (MissingFrameOutputResult != IElectraDecoder::EDecoderError::None)
	{
		return MissingFrameOutputResult;
	}

	// Update the current POC values.
	if (!BitstreamParamsH264.DPBPOC.UpdatePOC(InSliceInfos[0].NalUnitType, InSliceInfos[0].NalRefIdc, InSliceInfos[0].Header, InSequenceParameterSet))
	{
		PostError(0, FString::Printf(TEXT("DecodeSlicesH264() failed. %s"), *BitstreamParamsH264.DPBPOC.GetLastError()), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}

	// Set the reference frames.
	pp.UsedForReferenceFlags = 0;
	pp.NonExistingFrameFlags = 0;
	FMemory::Memzero(fdr->ReferenceFrameList);
	for(int32 i=0; i<UE_ARRAY_COUNT(pp.RefFrameList); ++i)
	{
		pp.RefFrameList[i].bPicEntry = 0xff;
		if (i < RefFrames.Num())
		{
			TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> refFrame;
			if (RefFrames[i].UserFrameInfo.IndexInBuffer >= 0)
			{
				refFrame = DPB->GetFrameAtIndex(RefFrames[i].UserFrameInfo.IndexInBuffer);
			}
			else
			{
				refFrame = MissingReferenceFrame;
			}
			if (refFrame.IsValid())
			{
				int32 dpbPos = refFrame->IndexInPictureBuffer;
				fdr->ReferenceFrameList[dpbPos] = refFrame->Texture.GetReference();
				pp.RefFrameList[i].Index7Bits = (UCHAR)dpbPos;
				pp.RefFrameList[i].AssociatedFlag = !RefFrames[i].bIsLongTerm ? 0 : 1;
				pp.UsedForReferenceFlags |= 3U << (i * 2);
				pp.FieldOrderCntList[i][0] = RefFrames[i].TopPOC;
				pp.FieldOrderCntList[i][1] = RefFrames[i].BottomPOC;
				pp.FrameNumList[i] = !RefFrames[i].bIsLongTerm ? RefFrames[i].FrameNum : RefFrames[i].LongTermFrameIndex;
				if (refFrame == MissingReferenceFrame)
				{
					pp.NonExistingFrameFlags |= (1 << i);
				}
			}
		}
	}
	pp.CurrFieldOrderCnt[0] = BitstreamParamsH264.DPBPOC.GetTopPOC();
	pp.CurrFieldOrderCnt[1] = BitstreamParamsH264.DPBPOC.GetBottomPOC();
	pp.frame_num = (USHORT) InSliceInfos[0].Header.frame_num;

	// Set the output frame.
	int32 dpbPos = TargetFrame->IndexInPictureBuffer;
	fdr->ReferenceFrameList[dpbPos] = TargetFrame->Texture.GetReference();
	pp.CurrPic.bPicEntry = (UCHAR)dpbPos;		// AssociatedFlag here would indicate this to be the bottom field

	// Quantization matrices
	DXVA_Qmatrix_H264& qm = input.QuantMtx;
	if (!InPictureParameterSet.pic_scaling_matrix_present_flag)
	{
		FMemory::Memcpy(qm.bScalingLists4x4, InSequenceParameterSet.ScalingList4x4, 6*16);
		FMemory::Memcpy(qm.bScalingLists8x8[0], InSequenceParameterSet.ScalingList8x8[0], 64);
		FMemory::Memcpy(qm.bScalingLists8x8[1], InSequenceParameterSet.ScalingList8x8[1], 64);
	}
	else
	{
		FMemory::Memcpy(qm.bScalingLists4x4, InPictureParameterSet.ScalingList4x4, 6*16);
		FMemory::Memcpy(qm.bScalingLists8x8[0], InPictureParameterSet.ScalingList8x8[0], 64);
		FMemory::Memcpy(qm.bScalingLists8x8[1], InPictureParameterSet.ScalingList8x8[1], 64);
	}

	D3D12_VIDEO_DECODE_INPUT_STREAM_ARGUMENTS isa {};
	isa.pHeap = CurrentConfig.VideoDecoderHeap;
	isa.NumFrameArguments = 0;
	isa.FrameArguments[isa.NumFrameArguments].Type = D3D12_VIDEO_DECODE_ARGUMENT_TYPE_PICTURE_PARAMETERS;
	isa.FrameArguments[isa.NumFrameArguments].Size = sizeof(pp);
	isa.FrameArguments[isa.NumFrameArguments].pData = &pp;
	++isa.NumFrameArguments;

	isa.FrameArguments[isa.NumFrameArguments].Type = D3D12_VIDEO_DECODE_ARGUMENT_TYPE_INVERSE_QUANTIZATION_MATRIX;
	isa.FrameArguments[isa.NumFrameArguments].Size = sizeof(qm);
	isa.FrameArguments[isa.NumFrameArguments].pData = &qm;
	++isa.NumFrameArguments;

	isa.FrameArguments[isa.NumFrameArguments].Type = D3D12_VIDEO_DECODE_ARGUMENT_TYPE_SLICE_CONTROL;
	isa.FrameArguments[isa.NumFrameArguments].Size = input.SliceHeaders.Num() * sizeof(DXVA_Slice_H264_Short);
	isa.FrameArguments[isa.NumFrameArguments].pData = input.SliceHeaders.GetData();
	++isa.NumFrameArguments;
	isa.CompressedBitstream.pBuffer = fdr->D3DBitstreamBuffer;
	isa.CompressedBitstream.Offset = 0;
	isa.CompressedBitstream.Size = fdr->D3DBitstreamBufferPayloadSize;

	isa.ReferenceFrames.NumTexture2Ds = FFrameDecodeResource::kMaxRefFrames;
	isa.ReferenceFrames.ppTexture2Ds = fdr->ReferenceFrameList;
	isa.ReferenceFrames.pSubresources = fdr->ReferenceFrameListSubRes;
#if PLATFORM_WINDOWS
	isa.ReferenceFrames.ppHeaps = nullptr;
#endif

	IElectraDecoder::EDecoderError decres = ExecuteCommonDecode(isa, osa);
	if (decres != IElectraDecoder::EDecoderError::None)
	{
		BitstreamParamsH264.DPBPOC.UndoPOCUpdate();
		return decres;
	}
	AutoRelease.ReleaseOwnership();

	// Add to the currently active list.
	fdr->D3DDecoder = VideoDecoder;
	fdr->D3DDecoderHeap = CurrentConfig.VideoDecoderHeap;
	//ActiveFrameDecodeResources.Emplace(MoveTemp(fdr));
	AvailableFrameDecodeResourceQueue.Enqueue(fdr);

	// Update the running frame number we use to associate this frame with.
	++RunningFrameNumLo;
	uint64 AssociatedUserValue = ((uint64)RunningFrameNumHi << 32) | (uint64)RunningFrameNumLo;

	// Create a new decoder output and set it up.
	TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe> InDec = MakeShared<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe>();
	InDec->PTS = InInputAccessUnit.PTS;
	InDec->UserValue = InInputAccessUnit.UserValue;
	InDec->OwningDPB = DPB;
	InDec->DecodedFrame = TargetFrame;
	InDec->UserValue0 = AssociatedUserValue;
	InDec->bDoNotOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) != EElectraDecoderFlags::None;
	InSequenceParameterSet.GetCrop(InDec->Crop.Left, InDec->Crop.Right, InDec->Crop.Top, InDec->Crop.Bottom);
	InDec->Width = InSequenceParameterSet.GetWidth();
	InDec->Height = InSequenceParameterSet.GetHeight();
	InDec->ImageWidth = InDec->Width - InDec->Crop.Left - InDec->Crop.Right;
	InDec->ImageHeight = InDec->Height - InDec->Crop.Top - InDec->Crop.Bottom;
	//InDec->Pitch = InDec->ImageWidth;
	InDec->NumBits = 8;
	InDec->BufferFormat = EPixelFormat::PF_NV12;
	InDec->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
	InSequenceParameterSet.GetAspect(InDec->AspectW, InDec->AspectH);
	auto FrameRate = InSequenceParameterSet.GetTiming();
	InDec->FrameRateN = FrameRate.Denom ? FrameRate.Num : 30;
	InDec->FrameRateD = FrameRate.Denom ? FrameRate.Denom : 1;
	InDec->Codec4CC = 'avcC';
	InDec->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("dx")));
	InDec->ExtraValues.Emplace(TEXT("dxversion"), FVariant((int64) 12000));
	InDec->ExtraValues.Emplace(TEXT("sw"), FVariant(false));
	InDec->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("avc")));
	InDec->ExtraValues.Emplace(TEXT("pixfmt"), FVariant((int64)EPixelFormat::PF_NV12));
	InDec->ExtraValues.Emplace(TEXT("pixenc"), FVariant((int64)EElectraTextureSamplePixelEncoding::Native));
	FramesInDecoder.Emplace(MoveTemp(InDec));

	// Update the simulation DPB with the new decoded frame.
	ElectraDecodersUtil::MPEG::H264::FOutputFrameInfo FrameInfo;
	FrameInfo.IndexInBuffer = TargetFrame->IndexInPictureBuffer;
	FrameInfo.PTS = InInputAccessUnit.PTS;
	FrameInfo.UserValue0 = AssociatedUserValue;
	BitstreamParamsH264.DPBPOC.EndFrame(OutputFrameInfos, UnrefFrameInfos, FrameInfo, InSliceInfos[0].NalUnitType, InSliceInfos[0].NalRefIdc, InSliceInfos[0].Header, false);
	return HandleOutputListH264(OutputFrameInfos);
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H264::HandleOutputListH264(const TArray<ElectraDecodersUtil::MPEG::H264::FOutputFrameInfo>& InOutputFrameInfos)
{
	TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe> InDec;
	for(int32 i=0; i<InOutputFrameInfos.Num(); ++i)
	{
		// In case the frame is a missing frame we ignore it.
		if (InOutputFrameInfos[i].IndexInBuffer < 0)
		{
			continue;
		}

//		UE_LOGF(LogD3D12VideoDecodersElectra, Log, "Output frame %d, %lld", InOutputFrameInfos[i].IndexInBuffer, (long long int)InOutputFrameInfos[i].PTS.GetTicks());
		TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> Frame = DPB->GetFrameAtIndex(InOutputFrameInfos[i].IndexInBuffer);
		check(Frame.IsValid());
		if (!Frame.IsValid())
		{
			PostError(0, FString::Printf(TEXT("HandleOutputListH264() failed. Output frame index is not valid for this DPB")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Locate the decoder output structure for this frame that we created earlier.
		for(int32 idx=0; idx<FramesInDecoder.Num(); ++idx)
		{
			if (FramesInDecoder[idx]->PTS == InOutputFrameInfos[i].PTS)
			{
				InDec = FramesInDecoder[idx];
				FramesInDecoder.RemoveAt(idx);
				break;
			}
		}
		if (!InDec.IsValid())
		{
			PostError(0, FString::Printf(TEXT("HandleOutputListH264() failed. Output frame not found in input list")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		check(InDec->OwningDPB == DPB);	// this should not trigger. A new DPB - if at all - should be created only when the decoder is flushed.
		check(InDec->DecodedFrame == Frame);
		check(InDec->UserValue0 == InOutputFrameInfos[i].UserValue0);
		if (!InDec->bDoNotOutput)
		{
			// Add to the ready-for-output queue.
			FramesReadyForOutput.Emplace(MoveTemp(InDec));
		}
		else
		{
			// Add to the queue of frames that were already output.
			// While this is not true we need to add it here and not back to the DPB because the
			// frame could still be referenced!
			FramesGivenOutForOutput.Emplace(MoveTemp(InDec));
		}
	}
	return IElectraDecoder::EDecoderError::None;
}


bool FD3D12VideoDecoder_H264::InternalResetToCleanStart()
{
	BitstreamParamsH264.Reset();
	return true;
}


}
