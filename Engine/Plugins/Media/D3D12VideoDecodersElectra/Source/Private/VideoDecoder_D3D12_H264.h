// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoDecoder_D3D12_Common.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"

namespace ElectraVideoDecodersD3D12Video
{

class FD3D12VideoDecoder_H264 : public FD3D12VideoDecoder
{
public:
	FD3D12VideoDecoder_H264(const FCodecFormatHelper::FCodecInfo& InCodecInfo, const D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InDecodeSupport, const TMap<FString, FVariant>& InOptions, const TRefCountPtr<ID3D12Device>& InD3D12Device, const TRefCountPtr<ID3D12VideoDevice>& InVideoDevice, uint32 InVideoDeviceNodeIndex);
	virtual ~FD3D12VideoDecoder_H264();

protected:
	ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override;
	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;

private:
	bool InternalResetToCleanStart() override;

	struct FBitstreamParamsH264
	{
		void Reset()
		{
			SPSs.Empty();
			PPSs.Empty();
			DPBPOC.Reset();
		}
		struct FSliceDecodeInfo
		{
			uint8 NalUnitType = 0;
			uint8 NalRefIdc = 0;
			// Parsed slice header.
			ElectraDecodersUtil::MPEG::H264::FSliceHeader Header;
			// Address of the nal unit byte of this slice.
			const uint8* NalUnitStartAddress = nullptr;
			// The number of bytes making up this slice, including the nal unit byte.
			uint32 NumBytesInSlice = 0;
		};

		TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet> SPSs;
		TMap<uint32, ElectraDecodersUtil::MPEG::H264::FPictureParameterSet> PPSs;

		ElectraDecodersUtil::MPEG::H264::FSlicePOCVars DPBPOC;
	};

	EDecoderError GetCodecSpecificDataH264(FBitstreamParamsH264& OutBitstreamParamsH264, const TMap<FString, FVariant>& InAdditionalOptions, bool bIsRequired);
	EDecoderError DecodeAccessUnitH264(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	EDecoderError DecodeSlicesH264(const FInputAccessUnit& InInputAccessUnit, const TArray<FBitstreamParamsH264::FSliceDecodeInfo>& InSliceInfos, const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet& InSequenceParameterSet, const ElectraDecodersUtil::MPEG::H264::FPictureParameterSet& InPictureParameterSet);
	EDecoderError HandleOutputListH264(const TArray<ElectraDecodersUtil::MPEG::H264::FOutputFrameInfo>& InOutputFrameInfos);

	FBitstreamParamsH264 BitstreamParamsH264;
};

}
