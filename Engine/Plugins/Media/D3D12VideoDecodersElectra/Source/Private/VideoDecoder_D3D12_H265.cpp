// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoder_D3D12_H265.h"
#include "D3D12VideoDecodersElectraModule.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H265.h"
#include "PixelFormat.h"


namespace ElectraVideoDecodersD3D12Video
{

const uint8 FD3D12VideoDecoder_H265::ScanOrderDiag4[16] = { 0, 4, 1, 8, 5, 2, 12, 9, 6, 3, 13, 10, 7, 14, 11, 15 };
const uint8 FD3D12VideoDecoder_H265::ScanOrderDiag8[64] = { 0, 8, 1, 16, 9, 2, 24, 17, 10, 3, 32, 25, 18, 11, 4, 40,
											33, 26, 19, 12, 5, 48, 41, 34, 27, 20, 13, 6, 56, 49, 42, 35,
											28, 21, 14, 7, 57, 50, 43, 36, 29, 22, 15, 58, 51, 44, 37, 30,
											23, 59, 52, 45, 38, 31, 60, 53, 46, 39, 61, 54, 47, 62, 55, 63 };

FD3D12VideoDecoder_H265::FD3D12VideoDecoder_H265(const FCodecFormatHelper::FCodecInfo& InCodecInfo, const D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InDecodeSupport, const TMap<FString, FVariant>& InOptions, const TRefCountPtr<ID3D12Device>& InD3D12Device, const TRefCountPtr<ID3D12VideoDevice>& InVideoDevice, uint32 InVideoDeviceNodeIndex)
	: FD3D12VideoDecoder(InCodecInfo, InDecodeSupport, InOptions, InD3D12Device, InVideoDevice, InVideoDeviceNodeIndex)
{
}

FD3D12VideoDecoder_H265::~FD3D12VideoDecoder_H265()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

IElectraDecoder::ECSDCompatibility FD3D12VideoDecoder_H265::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// No decoder yet means we are compatible.
	if (!VideoDecoder.IsValid())
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	FBitstreamParamsH265 temp;
	if (GetCodecSpecificDataH265(temp, CSDAndAdditionalOptions, false) == IElectraDecoder::EDecoderError::Error)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	// We can only check against a single provided SPS. If none or several, start over.
	if (temp.SPSs.Num() != 1)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}

#if 0
	const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet sps = temp.SPSs.CreateConstIterator().Value();
	// Check that the new CSD isn't Main10 when we are only Main.
	if (sps.profile_tier_level.general_profile_idc == 2 && !CodecInfo.b10Bit)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	int32 NewWidth, NewHeight, NewDPBSize;
	NewDPBSize = sps.GetDPBSize();
	sps.GetDisplaySize(NewWidth, NewHeight);
	if (NewDPBSize > CurrentConfig.MaxNumInDPB ||
		NewWidth > CurrentConfig.MaxDecodedWidth ||
		NewHeight > CurrentConfig.MaxDecodedHeight)
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	return IElectraDecoder::ECSDCompatibility::Compatible;
#else
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
#endif
}

TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> FD3D12VideoDecoder_H265::CreateBitstreamProcessor()
{
	TMap<FString, FVariant> DecoderFeatures;
	GetFeatures(DecoderFeatures);
	return FElectraDecoderBitstreamProcessorH265::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
}


IElectraDecoder::EDecoderError FD3D12VideoDecoder_H265::GetCodecSpecificDataH265(FBitstreamParamsH265& OutBitstreamParamsH265, const TMap<FString, FVariant>& InAdditionalOptions, bool bIsRequired)
{
	TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("csd"));
	if (CSD == OutBitstreamParamsH265.CurrentCSD)
	{
		return IElectraDecoder::EDecoderError::None;
	}
	// Split the CSD into individual NAL units.
	TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NalUnits;
	if (!ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NalUnits, CSD.GetData(), CSD.Num()))
	{
		if (bIsRequired)
		{
			PostError(0, TEXT("Failed to locate the NALUs in the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
			return IElectraDecoder::EDecoderError::Error;
		}
		else
		{
			return IElectraDecoder::EDecoderError::NoBuffer;
		}
	}
	// Parse the VPS, SPS and PPS
	for(int32 i=0; i<NalUnits.Num(); ++i)
	{
		if (NalUnits[i].Type == 32)
		{
			if (!ElectraDecodersUtil::MPEG::H265::ParseVideoParameterSet(OutBitstreamParamsH265.VPSs, CSD.GetData() + NalUnits[i].Offset + NalUnits[i].UnitLength, NalUnits[i].Size))
			{
				PostError(0, TEXT("Failed to parse the VPS from the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else if (NalUnits[i].Type == 33)
		{
			if (!ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(OutBitstreamParamsH265.SPSs, CSD.GetData() + NalUnits[i].Offset + NalUnits[i].UnitLength, NalUnits[i].Size))
			{
				PostError(0, TEXT("Failed to parse the SPS from the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else if (NalUnits[i].Type == 34)
		{
			if (!ElectraDecodersUtil::MPEG::H265::ParsePictureParameterSet(OutBitstreamParamsH265.PPSs, OutBitstreamParamsH265.SPSs, CSD.GetData() + NalUnits[i].Offset + NalUnits[i].UnitLength, NalUnits[i].Size))
			{
				PostError(0, TEXT("Failed to parse the PPS from the codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
	}
	OutBitstreamParamsH265.CurrentCSD = MoveTemp(CSD);
	return IElectraDecoder::EDecoderError::None;
}



IElectraDecoder::EDecoderError FD3D12VideoDecoder_H265::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (!InInputAccessUnit.Data || !InInputAccessUnit.DataSize)
	{
		return IElectraDecoder::EDecoderError::None;
	}

	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
	{
		if (GetCodecSpecificDataH265(BitstreamParamsH265, InAdditionalOptions, true) != IElectraDecoder::EDecoderError::None)
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// We need to isolate the slices that make up this frame.
	TArray<FBitstreamParamsH265::FSliceDecodeInfo> SliceInfos;
	// Go over each of the NALUs in the bitstream.
	const uint8* Start = (const uint8*)InInputAccessUnit.Data;
	const uint8* End = Start + InInputAccessUnit.DataSize;
	bool bIsFirstNALU = true;
	/**
	 * Check if an EOS or EOB NALU is present in this access unit.
	 * If present it should be at the end, but it could also appear first in the subsequent access unit in
	 * which case it must be applied first as if it had appeared last in the previous access unit.
	 */
	enum class ESequenceEnd
	{
		None,
		AtStart,
		AtEnd
	};
	ESequenceEnd SequenceEnd = ESequenceEnd::None;
	while(Start < End)
	{
		uint32 NaluLength = ((uint32)Start[0] << 24) | ((uint32)Start[1] << 16) | ((uint32)Start[2] << 8) | ((uint32)Start[3]);
		uint32 nuh = (uint32)Start[4] << 8 | Start[5];
		uint32 NalUnitType = nuh >> 9;
		uint32 NuhLayerId = (nuh >> 3) & 63;
		uint32 NumTemporalIdPlus1 = nuh & 7;

		if ((NalUnitType >= 0 && NalUnitType <= 9) || (NalUnitType >= 16 && NalUnitType <= 21))
		{
			FBitstreamParamsH265::FSliceDecodeInfo& SliceInfo = SliceInfos.Emplace_GetRef();
			SliceInfo.NalUnitType = (uint8)NalUnitType;
			SliceInfo.NuhLayerId = (uint8)NuhLayerId;
			SliceInfo.NumTemporalIdPlus1 = (uint8)NumTemporalIdPlus1;
			ElectraDecodersUtil::MPEG::H265::FBitstreamReaderH265 br;
			TUniquePtr<ElectraDecodersUtil::MPEG::H265::FRBSP> SliceRBSP;
			if (!ElectraDecodersUtil::MPEG::H265::ParseSliceHeader(SliceRBSP, br, SliceInfo.Header, BitstreamParamsH265.VPSs, BitstreamParamsH265.SPSs, BitstreamParamsH265.PPSs, Start+4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream slice header"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
			// Fill in the remaining slice information
			SliceInfo.NalUnitStartAddress = Start + 4;
			SliceInfo.NumBytesInSlice = NaluLength;
		}
		else if (NalUnitType == 32)
		{
			if (!ElectraDecodersUtil::MPEG::H265::ParseVideoParameterSet(BitstreamParamsH265.VPSs, Start + 4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream inband VPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		// Inband SPS
		else if (NalUnitType == 33)
		{
			if (!ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(BitstreamParamsH265.SPSs, Start + 4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream inband SPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		// Inband PPS
		else if (NalUnitType == 34)
		{
			if (!ElectraDecodersUtil::MPEG::H265::ParsePictureParameterSet(BitstreamParamsH265.PPSs, BitstreamParamsH265.SPSs, Start + 4, NaluLength))
			{
				PostError(0, TEXT("Failed to parse bitstream inband PPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		// EOS or EOB?
		else if (NalUnitType == 36 || NalUnitType == 37)
		{
			SequenceEnd = bIsFirstNALU ? ESequenceEnd::AtStart : ESequenceEnd::AtEnd;
		}
		Start += NaluLength + 4;
		bIsFirstNALU = false;
	}
	// Apply any EOS/EOB NALU found at the start right away.
	if (SequenceEnd == ESequenceEnd::AtStart)
	{
		BitstreamParamsH265.bIsFirstInSequence = true;
	}

	// Any slices to decode?
	if (SliceInfos.Num())
	{
		// Create a new decoder if we do not have one. This does not require any information about the resolution or DPB.
		if (!VideoDecoder.IsValid())
		{
			if (!InternalDecoderCreate())
			{
				return IElectraDecoder::EDecoderError::Error;
			}
		}

		const ElectraDecodersUtil::MPEG::H265::FPictureParameterSet* ppsPtr = BitstreamParamsH265.PPSs.Find(SliceInfos[0].Header.slice_pic_parameter_set_id);
		if (!ppsPtr)
		{
			PostError(0, TEXT("Reference picture parameter set not found"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet* spsPtr = BitstreamParamsH265.SPSs.Find(ppsPtr->pps_seq_parameter_set_id);
		if (!spsPtr)
		{
			PostError(0, TEXT("Reference sequence parameter set not found"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		const ElectraDecodersUtil::MPEG::H265::FVideoParameterSet* vpsPtr = BitstreamParamsH265.VPSs.Find(spsPtr->sps_video_parameter_set_id);
		if (!vpsPtr)
		{
			PostError(0, TEXT("Reference video parameter set not found"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		// On an IRAP frame check if we need a new decoder, either because we have none or the relevant
		// decoding parameters changed.
		if (SliceInfos[0].Header.bIsIRAP)
		{
			const int32 DPBSize = spsPtr->GetDPBSize();
			int32 dw, dh;
			spsPtr->GetDisplaySize(dw, dh);

			// Check if the decoder heap parameters have changed such that we have to create a new one.
			if (DPBSize != CurrentConfig.MaxNumInDPB || dw != CurrentConfig.VideoDecoderDPBWidth || dh != CurrentConfig.VideoDecoderDPBHeight)
			{
				CurrentConfig.VideoDecoderHeap.SafeRelease();
			}
			if (!CurrentConfig.VideoDecoderHeap.IsValid())
			{
				if (!CreateDecoderHeap(DPBSize, dw, dh, spsPtr->GetMinCbSizeY()))
				{
					return IElectraDecoder::EDecoderError::Error;
				}
			}

			if (!DPB.IsValid())
			{
				// As far as the decoded frames go, their size can be the maximum that is required
				// for this stream (the largest resolution).
				const int32 Width = (int32) DecodeSupport.Width;
				const int32 Height = (int32) DecodeSupport.Height;
				const int32 NumFrames = DPBSize + 2;	// 1 extra for the current frame that's not in the DPB yet, and 1 extra that acts as a 'missing' frame.
				if (!CreateDPB(DPB, Width, Height, GetFrameAlignment(), NumFrames))
				{
					return IElectraDecoder::EDecoderError::Error;
				}
				MissingReferenceFrame = DPB->GetNextUnusedFrame();
				if (!MissingReferenceFrame.IsValid())
				{
					PostError(0, TEXT("Could not create empty frame used to fill in for missing frames"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
					return IElectraDecoder::EDecoderError::Error;
				}
			}
			// Remember the type of IRAP (see below).
			BitstreamParamsH265.AssociatedIRAP = SliceInfos[0].NalUnitType;
			++BitstreamParamsH265.IRAPCount;
		}
		/*
			Filter out any RASL pictures for which the associated IRAP, that is the first coded picture, is a CRA or BLA.

			We do this because the DXVA HEVC spec states:
				"The host decoder is recommended to send only decodable pictures to accelerator. For example, when the associated IRAP picture
				is a BLA picture or is a CRA picture that is the first coded picture in the bitstream, the host decoder is recommended not to
				send the associated RASL pictures to accelerator if any such RASL pictures are present in the bitstream. 
				However, the accelerator shall be robust enough to handle any non-decodable pictures."

			And the "However" statement does not hold true. There are accelerators where sending down these pictures will produce
			garbage frames. So we need to filter them out.
		*/
		if (BitstreamParamsH265.IRAPCount <= 1 && (BitstreamParamsH265.AssociatedIRAP == 21 || BitstreamParamsH265.AssociatedIRAP == 16 || BitstreamParamsH265.AssociatedIRAP == 17 || BitstreamParamsH265.AssociatedIRAP == 18))
		{
			for(int32 i=0; i<SliceInfos.Num(); ++i)
			{
				if (SliceInfos[i].Header.NalUnitType == 8 || SliceInfos[i].Header.NalUnitType == 9)
				{
					SliceInfos.RemoveAt(i);
					--i;
				}
			}
		}

		IElectraDecoder::EDecoderError Error = DecodeSlicesH265(InInputAccessUnit, SliceInfos, *spsPtr, *ppsPtr);
		// When the frame was decoded and there is an EOS/EOB NALU present we need to apply it now.
		if (Error == IElectraDecoder::EDecoderError::None && SequenceEnd == ESequenceEnd::AtEnd)
		{
			BitstreamParamsH265.bIsFirstInSequence = true;
		}
		return Error;
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H265::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bIsDraining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bIsDraining = true;
	TArray<ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::FDPBOutputFrame> OutputFrames;
	BitstreamParamsH265.DPB.Flush(OutputFrames);
	BitstreamParamsH265.bIsFirstInSequence = true;
	BitstreamParamsH265.AssociatedIRAP = -1;
	BitstreamParamsH265.IRAPCount = 0;
	return HandleOutputListH265(OutputFrames);
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H265::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Wait for a while for the most recent decode operation to have finished.
	if (VideoDecoderSync.IsValid())
	{
		VideoDecoderSync->AwaitCompletion(500);
	}
	BitstreamParamsH265.Reset();
	ReturnAllFrames();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H265::DecodeSlicesH265(const FInputAccessUnit& InInputAccessUnit, const TArray<FBitstreamParamsH265::FSliceDecodeInfo>& InSliceInfos, const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& InSequenceParameterSet, const ElectraDecodersUtil::MPEG::H265::FPictureParameterSet& InPictureParameterSet)
{
	// When there are no slices we are done.
	if (InSliceInfos.IsEmpty())
	{
		return IElectraDecoder::EDecoderError::None;
	}

	if (!VideoDecoderSync.IsValid())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// The previous operation must have completed, primarily because we (may) need the decoded frame from before
	// as a reference frame for this call and that frame thus needs to have finished.
	if (!VideoDecoderSync->AwaitCompletion(500))
	{
		UE_LOGF(LogD3D12VideoDecodersElectra, Warning, "DecodeSlicesH265() waited too long for the previous operation to complete. Trying again later.");
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Some capability checks
	if (InSequenceParameterSet.sps_extension_present_flag || InPictureParameterSet.pps_extension_present_flag)
	{
		if (InSequenceParameterSet.sps_range_extension_flag || InPictureParameterSet.pps_range_extension_flag)
		{
			PostError(0, FString::Printf(TEXT("DecodeSlicesH265() failed. Cannot decode streams using range extensions (RExt)")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		else if (InSequenceParameterSet.sps_multilayer_extension_flag || InPictureParameterSet.pps_multilayer_extension_flag)
		{
			PostError(0, FString::Printf(TEXT("DecodeSlicesH265() failed. Cannot decode streams using multilayer extensions")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		else if (InSequenceParameterSet.sps_3d_extension_flag || InPictureParameterSet.pps_3d_extension_flag)
		{
			PostError(0, FString::Printf(TEXT("DecodeSlicesH265() failed. Cannot decode streams using 3D extensions")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		else if (InSequenceParameterSet.sps_scc_extension_flag || InPictureParameterSet.pps_scc_extension_flag)
		{
			PostError(0, FString::Printf(TEXT("DecodeSlicesH265() failed. Cannot decode streams using screen content coding extensions")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	check(DPB.IsValid());
	if (!DPB.IsValid())
	{
		PostError(0, FString::Printf(TEXT("DecodeSlicesH265() failed. There is no DPB")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}


	// Get the frames that are currently referenced by the DPB.
	TArray<ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::FReferenceFrameListEntry> ReferenceFrames;
	TArray<int32> DPBIndexLists[ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::EList::eMAX];
	BitstreamParamsH265.DPB.GetReferenceFramesFromDPB(ReferenceFrames, DPBIndexLists);

	// Go over all the frames that we have already handed out for display.
	// These should have been copied or converted the moment we handed them out and are thus
	// available for use again, provided the DPB does not still need them for reference.
	for(int32 i=0; i<FramesGivenOutForOutput.Num(); ++i)
	{
		// FIXME: If we have to create a new DPB then the frames we handed out may be from the old DPB
		check(FramesGivenOutForOutput[i]->OwningDPB == DPB);
		bool bNoLongerReferenced = true;
		for(int32 j=0; j<ReferenceFrames.Num(); ++j)
		{
			if (FramesGivenOutForOutput[i]->UserValue0 == ReferenceFrames[j].UserFrameInfo.UserValue0)
			{
				bNoLongerReferenced = false;
				break;
			}
		}
		if (bNoLongerReferenced)
		{
			FramesGivenOutForOutput[i]->OwningDPB->ReturnFrameToAvailableQueue(MoveTemp(FramesGivenOutForOutput[i]->DecodedFrame));
			FramesGivenOutForOutput.RemoveAt(i);
			--i;
		}
	}

	// Get a target frame to decode into.
	TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> TargetFrame = DPB->GetNextUnusedFrame();
	if (!TargetFrame.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
	FAutoReturnUnusedFrame AutoRelease(DPB, TargetFrame);

	// Get an available frame decode resource
	TSharedPtr<FFrameDecodeResource, ESPMode::ThreadSafe> fdr;
	if (!AvailableFrameDecodeResourceQueue.Dequeue(fdr))
	{
		fdr = MakeShared<FFrameDecodeResource, ESPMode::ThreadSafe>();
	}
	// Make sure the input is set to the correct type.
	if (fdr->PicInput.GetIndex() != fdr->PicInput.IndexOfType<FFrameDecodeResource::FInputH265>())
	{
		fdr->PicInput.Emplace<FFrameDecodeResource::FInputH265>(FFrameDecodeResource::FInputH265());
	}
	// Calculate the total input bitstream size.
	uint32 TotalSliceSize = 0;
	for(auto& si : InSliceInfos)
	{
		// Need to prepend each slice with a 0x000001 startcode and each slice must be zero-padded to 128 byte alignment.
		uint32 SliceSize = Align(si.NumBytesInSlice + 3, 128);
		TotalSliceSize += SliceSize;
	}
	// If necessary reallocate the bitstream buffer.
	if (!PrepareBitstreamBuffer(fdr, TotalSliceSize))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	FFrameDecodeResource::FInputH265& input = fdr->PicInput.Get<FFrameDecodeResource::FInputH265>();
	input.SliceHeaders.SetNumUninitialized(InSliceInfos.Num());

	// Copy the slices into the buffer and set up the short slice headers
	HRESULT Result;
	uint8* BufferPtr = nullptr;
	if ((Result = fdr->D3DBitstreamBuffer->Map(0, nullptr, (void**)&BufferPtr)) != S_OK)
	{
		PostError(0, TEXT("ID3D12Resource::Map() failed for bitstream buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
		return IElectraDecoder::EDecoderError::Error;
	}
	uint8* BufferBase = BufferPtr;
	for(int32 ns=0; ns<InSliceInfos.Num(); ++ns)
	{
		uint32 SliceSize = 3 + InSliceInfos[ns].NumBytesInSlice;
		input.SliceHeaders[ns].BSNALunitDataLocation = BufferPtr - BufferBase;
		input.SliceHeaders[ns].SliceBytesInBuffer = SliceSize;
		input.SliceHeaders[ns].wBadSliceChopping = 0;
		*BufferPtr++ = 0;
		*BufferPtr++ = 0;
		*BufferPtr++ = 1;
		FMemory::Memcpy(BufferPtr, InSliceInfos[ns].NalUnitStartAddress, InSliceInfos[ns].NumBytesInSlice);
		BufferPtr += InSliceInfos[ns].NumBytesInSlice;
		if (SliceSize != Align(SliceSize, 128))
		{
			uint32 PaddingSize = 128 - (SliceSize & 127);
			FMemory::Memzero(BufferPtr, PaddingSize);
			BufferPtr += PaddingSize;
		}
	}
	check(BufferPtr - BufferBase == TotalSliceSize);
	fdr->D3DBitstreamBuffer->Unmap(0, nullptr);
	fdr->D3DBitstreamBufferPayloadSize = BufferPtr - BufferBase;
	FMemory::Memzero(fdr->ReferenceFrameList);

	D3D12_VIDEO_DECODE_OUTPUT_STREAM_ARGUMENTS osa {};
	osa.pOutputTexture2D = TargetFrame->Texture;
    osa.OutputSubresource = 0;
	osa.ConversionArguments.Enable = false;

	// Input picture parameters
	DXVA_PicParams_HEVC& pp = input.PicParams;
	FMemory::Memzero(pp);
	pp.PicWidthInMinCbsY = (USHORT) InSequenceParameterSet.PicWidthInMinCbsY;
	pp.PicHeightInMinCbsY = (USHORT) InSequenceParameterSet.PicHeightInMinCbsY;
	pp.chroma_format_idc = InSequenceParameterSet.chroma_format_idc;
	pp.separate_colour_plane_flag = InSequenceParameterSet.separate_colour_plane_flag;
	pp.bit_depth_luma_minus8 = InSequenceParameterSet.bit_depth_luma_minus8;
	pp.bit_depth_chroma_minus8 = InSequenceParameterSet.bit_depth_chroma_minus8;
	pp.log2_max_pic_order_cnt_lsb_minus4 = InSequenceParameterSet.log2_max_pic_order_cnt_lsb_minus4;
	//pp.NoPicReorderingFlag = InSequenceParameterSet.sps_max_num_reorder_pics == 0;
	//pp.NoBiPredFlag = 0;
	pp.sps_max_dec_pic_buffering_minus1 = (UCHAR) InSequenceParameterSet.sps_max_dec_pic_buffering_minus1[InSequenceParameterSet.sps_max_sub_layers_minus1];
	pp.log2_min_luma_coding_block_size_minus3 = (UCHAR) InSequenceParameterSet.log2_min_luma_coding_block_size_minus3;
	pp.log2_diff_max_min_luma_coding_block_size = (UCHAR) InSequenceParameterSet.log2_diff_max_min_luma_coding_block_size;
	pp.log2_min_transform_block_size_minus2 = (UCHAR) InSequenceParameterSet.log2_min_luma_transform_block_size_minus2;
	pp.log2_diff_max_min_transform_block_size = (UCHAR) InSequenceParameterSet.log2_diff_max_min_luma_transform_block_size;
	pp.max_transform_hierarchy_depth_inter = (UCHAR) InSequenceParameterSet.max_transform_hierarchy_depth_inter;
	pp.max_transform_hierarchy_depth_intra = (UCHAR) InSequenceParameterSet.max_transform_hierarchy_depth_intra;
	pp.num_short_term_ref_pic_sets = (UCHAR) InSequenceParameterSet.num_short_term_ref_pic_sets;
	pp.num_long_term_ref_pics_sps = (UCHAR) InSequenceParameterSet.num_long_term_ref_pics_sps;
	pp.num_ref_idx_l0_default_active_minus1 = (UCHAR) InPictureParameterSet.num_ref_idx_l0_default_active_minus1;
	pp.num_ref_idx_l1_default_active_minus1 = (UCHAR) InPictureParameterSet.num_ref_idx_l1_default_active_minus1;
	pp.init_qp_minus26 = (CHAR) InPictureParameterSet.init_qp_minus26;
	if (InSliceInfos[0].Header.short_term_ref_pic_set_sps_flag == 0)
	{
		pp.ucNumDeltaPocsOfRefRpsIdx = (UCHAR) InSliceInfos[0].Header.st_ref_pic_set.NumDeltaPocsInSliceReferencedSet;
		pp.wNumBitsForShortTermRPSInSlice = (USHORT) InSliceInfos[0].Header.NumBitsForShortTermRefs;
	}

	pp.scaling_list_enabled_flag = InSequenceParameterSet.scaling_list_enabled_flag;
	pp.amp_enabled_flag = InSequenceParameterSet.amp_enabled_flag;
	pp.sample_adaptive_offset_enabled_flag = InSequenceParameterSet.sample_adaptive_offset_enabled_flag;
	if ((pp.pcm_enabled_flag = InSequenceParameterSet.pcm_enabled_flag) != 0)
	{
		pp.pcm_sample_bit_depth_luma_minus1 = InSequenceParameterSet.pcm_sample_bit_depth_luma_minus1;
		pp.pcm_sample_bit_depth_chroma_minus1 = InSequenceParameterSet.pcm_sample_bit_depth_chroma_minus1;
		pp.log2_min_pcm_luma_coding_block_size_minus3 = InSequenceParameterSet.log2_min_pcm_luma_coding_block_size_minus3;
		pp.log2_diff_max_min_pcm_luma_coding_block_size = InSequenceParameterSet.log2_diff_max_min_pcm_luma_coding_block_size;
		pp.pcm_loop_filter_disabled_flag = InSequenceParameterSet.pcm_loop_filter_disabled_flag;
	}
	pp.long_term_ref_pics_present_flag = InSequenceParameterSet.long_term_ref_pics_present_flag;
	pp.sps_temporal_mvp_enabled_flag = InSequenceParameterSet.sps_temporal_mvp_enabled_flag;
	pp.strong_intra_smoothing_enabled_flag = InSequenceParameterSet.strong_intra_smoothing_enabled_flag;
	pp.dependent_slice_segments_enabled_flag = InPictureParameterSet.dependent_slice_segments_enabled_flag;
	pp.output_flag_present_flag = InPictureParameterSet.output_flag_present_flag;
	pp.num_extra_slice_header_bits = InPictureParameterSet.num_extra_slice_header_bits;
	pp.sign_data_hiding_enabled_flag = InPictureParameterSet.sign_data_hiding_enabled_flag;
	pp.cabac_init_present_flag = InPictureParameterSet.cabac_init_present_flag;
	pp.constrained_intra_pred_flag = InPictureParameterSet.constrained_intra_pred_flag;
	pp.transform_skip_enabled_flag = InPictureParameterSet.transform_skip_enabled_flag;
	pp.cu_qp_delta_enabled_flag = InPictureParameterSet.cu_qp_delta_enabled_flag;
	pp.pps_slice_chroma_qp_offsets_present_flag = InPictureParameterSet.pps_slice_chroma_qp_offsets_present_flag;
	pp.weighted_pred_flag = InPictureParameterSet.weighted_pred_flag;
	pp.weighted_bipred_flag = InPictureParameterSet.weighted_bipred_flag;
	pp.transquant_bypass_enabled_flag = InPictureParameterSet.transquant_bypass_enabled_flag;
	pp.tiles_enabled_flag = InPictureParameterSet.tiles_enabled_flag;
	pp.entropy_coding_sync_enabled_flag = InPictureParameterSet.entropy_coding_sync_enabled_flag;
	pp.uniform_spacing_flag = InPictureParameterSet.uniform_spacing_flag;
	pp.pps_loop_filter_across_slices_enabled_flag = InPictureParameterSet.pps_loop_filter_across_slices_enabled_flag;
	pp.deblocking_filter_override_enabled_flag = InPictureParameterSet.deblocking_filter_override_enabled_flag;
	pp.pps_deblocking_filter_disabled_flag = InPictureParameterSet.pps_deblocking_filter_disabled_flag;
	pp.lists_modification_present_flag = InPictureParameterSet.lists_modification_present_flag;
	pp.slice_segment_header_extension_present_flag = InPictureParameterSet.slice_segment_header_extension_present_flag;
	pp.IrapPicFlag = InSliceInfos[0].Header.bIsIRAP ? 1 : 0;
	pp.IdrPicFlag = InSliceInfos[0].Header.bIsIDR ? 1 : 0;
	pp.IntraPicFlag = InSliceInfos[0].Header.bIsIRAP ? 1 : 0;
	pp.pps_cb_qp_offset = (CHAR) InPictureParameterSet.pps_cb_qp_offset;
	pp.pps_cr_qp_offset = (CHAR) InPictureParameterSet.pps_cr_qp_offset;
	if (InPictureParameterSet.tiles_enabled_flag)
	{
		pp.loop_filter_across_tiles_enabled_flag = InPictureParameterSet.loop_filter_across_tiles_enabled_flag;
		// CAUTION: The maximum number of tiles in the structure is set to accommodate
		//          at most level 6.3 with 20x22 tiles. Level 7 and higher allows for 40x44 !!
		check(InPictureParameterSet.num_tile_columns_minus1 < 20);
		check(InPictureParameterSet.num_tile_rows_minus1 < 22);
		pp.num_tile_columns_minus1 = (UCHAR) InPictureParameterSet.num_tile_columns_minus1;
		pp.num_tile_rows_minus1 = (UCHAR) InPictureParameterSet.num_tile_rows_minus1;
		if (!InPictureParameterSet.uniform_spacing_flag)
		{
			for(int32 i=0; i<InPictureParameterSet.column_width_minus1.Num(); ++i)
			{
				pp.column_width_minus1[i] = (USHORT) InPictureParameterSet.column_width_minus1[i];
			}
			for(int32 i=0; i<InPictureParameterSet.row_height_minus1.Num(); ++i)
			{
				pp.row_height_minus1[i] = (USHORT) InPictureParameterSet.row_height_minus1[i];
			}
		}
	}
	pp.diff_cu_qp_delta_depth = (UCHAR) InPictureParameterSet.diff_cu_qp_delta_depth;
	pp.pps_beta_offset_div2 = (CHAR) InPictureParameterSet.pps_beta_offset_div2;
	pp.pps_tc_offset_div2 = (CHAR) InPictureParameterSet.pps_tc_offset_div2;
	pp.log2_parallel_merge_level_minus2 = (UCHAR) InPictureParameterSet.log2_parallel_merge_level_minus2;
	if (++StatusReportFeedbackNumber == 0)
	{
		StatusReportFeedbackNumber = 1;
	}
	pp.StatusReportFeedbackNumber = StatusReportFeedbackNumber;


	// Update the POC values and simulation DPB and get the list of frames ready for output.
	TArray<ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::FDPBOutputFrame> OutputFrames;
	BitstreamParamsH265.DPB.ProcessFirstSliceOfFrame(OutputFrames, InSliceInfos[0].Header, BitstreamParamsH265.bIsFirstInSequence);
	// Get the list of reference frames needed to decode this frame.
	BitstreamParamsH265.DPB.GetReferenceFramesFromDPB(ReferenceFrames, DPBIndexLists);

	// Set the output frame.
	int32 pbIdx = TargetFrame->IndexInPictureBuffer;
	fdr->ReferenceFrameList[pbIdx] = TargetFrame->Texture.GetReference();
	pp.CurrPic.bPicEntry = (UCHAR)pbIdx;		// AssociatedFlag here has no meaning.
	pp.CurrPicOrderCntVal = BitstreamParamsH265.DPB.GetSlicePOC();

	// Set up the reference frames
	int32 BufferIndexOfMissingFrame = MissingReferenceFrame->IndexInPictureBuffer;
	for(int32 i=0; i<UE_ARRAY_COUNT(pp.RefPicList); ++i)
	{
		pp.RefPicList[i].bPicEntry = 0xff;
		if (i < ReferenceFrames.Num())
		{
			check(ReferenceFrames[i].bIsShortTermReference || ReferenceFrames[i].bIsLongTermReference);
			TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> refFrame;
			if (ReferenceFrames[i].UserFrameInfo.IndexInBuffer >= 0)
			{
				refFrame= DPB->GetFrameAtIndex(ReferenceFrames[i].UserFrameInfo.IndexInBuffer);
			}
			else
			{
				refFrame = MissingReferenceFrame;
			}
			if (refFrame.IsValid())
			{
				pbIdx = refFrame->IndexInPictureBuffer;
				fdr->ReferenceFrameList[pbIdx] = refFrame->Texture.GetReference();
				pp.RefPicList[i].Index7Bits = (UCHAR) pbIdx;
				pp.RefPicList[i].AssociatedFlag = ReferenceFrames[i].bIsLongTermReference ? 1 : 0;
				pp.PicOrderCntValList[i] = ReferenceFrames[i].POC;
			}
		}
	}
	// Set up the reference lists
	for(int32 nList=0; nList<3; ++nList)
	{
		UCHAR* RefPicSetXX = nList==0 ? pp.RefPicSetStCurrBefore : nList==1 ? pp.RefPicSetStCurrAfter : pp.RefPicSetLtCurr;
		// Preset the list with 0xff to indicate unused entries.
		FMemory::Memset(RefPicSetXX, 0xff, 8);
		const TArray<int32>& SrcListXX(nList==0 ? DPBIndexLists[ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::EList::eStCurrBefore] :
									   nList==1 ? DPBIndexLists[ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::EList::eStCurrAfter] :
												  DPBIndexLists[ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::EList::eLtCurr]);
		check(SrcListXX.Num() <= 8);
		for(int32 i=0; i<SrcListXX.Num(); ++i)
		{
			const ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::FReferenceFrameListEntry* DpbEntry = BitstreamParamsH265.DPB.GetDPBEntryAtIndex(SrcListXX[i]);
			if (!DpbEntry)
			{
				PostError(0, TEXT("DecodeSlicesH265() failed. DPB entry not found!"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
			bool bFound = false;
			for(int32 j=0; j<UE_ARRAY_COUNT(pp.RefPicList); ++j)
			{
				if (pp.RefPicList[j].Index7Bits == DpbEntry->UserFrameInfo.IndexInBuffer || pp.RefPicList[j].Index7Bits == BufferIndexOfMissingFrame)
				{
					bFound = true;
					*RefPicSetXX++ = (UCHAR)j;
					break;
				}
			}
			if (!bFound)
			{
				PostError(0, TEXT("DecodeSlicesH265() failed. DPB entry not found!"), ERRCODE_INTERNAL_FAILED_TO_DECODE);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
	}

	D3D12_VIDEO_DECODE_INPUT_STREAM_ARGUMENTS isa {};
	isa.pHeap = CurrentConfig.VideoDecoderHeap;
	isa.NumFrameArguments = 0;
	isa.FrameArguments[isa.NumFrameArguments].Type = D3D12_VIDEO_DECODE_ARGUMENT_TYPE_PICTURE_PARAMETERS;
	isa.FrameArguments[isa.NumFrameArguments].Size = sizeof(pp);
	isa.FrameArguments[isa.NumFrameArguments].pData = &pp;
	++isa.NumFrameArguments;

	// Send scaling list only when enabled and present in either SPS or PPS.
	if (InSequenceParameterSet.scaling_list_enabled_flag)
	{
		// Target quantization matrices
		DXVA_Qmatrix_HEVC& qm = input.QuantMtx;
		// Select the appropriate source
		const ElectraDecodersUtil::MPEG::H265::FScalingListData& sld = InPictureParameterSet.pps_scaling_list_data_present_flag ? InPictureParameterSet.scaling_list_data : InSequenceParameterSet.scaling_list_data;

		// The matrices we get are in diagonal order, but DXVA2 expects them in linear order, so we need
		// to reorder the elements as we copy them over.
		// Copy scale factors
		for(int32 i=0; i<6; ++i)
		{
			for(int32 j=0; j<16; ++j)
			{
				qm.ucScalingLists0[i][j] = sld.scaling_list[0][i][ScanOrderDiag4[j]];
			}
			for(int32 j=0; j<64; ++j)
			{
				qm.ucScalingLists1[i][j] = sld.scaling_list[1][i][ScanOrderDiag8[j]];
				qm.ucScalingLists2[i][j] = sld.scaling_list[2][i][ScanOrderDiag8[j]];
				if (i < 2)
				{
					qm.ucScalingLists3[i][j] = sld.scaling_list[3][i * 3][ScanOrderDiag8[j]];
				}
			}
			// Copy DC coefficients from list 2
			qm.ucScalingListDCCoefSizeID2[i] = sld.scaling_list_dc[2][i];
		}
		// Copy DC coefficients from list 3
		qm.ucScalingListDCCoefSizeID3[0] = sld.scaling_list_dc[3][0];
		qm.ucScalingListDCCoefSizeID3[1] = sld.scaling_list_dc[3][3];

		isa.FrameArguments[isa.NumFrameArguments].Type = D3D12_VIDEO_DECODE_ARGUMENT_TYPE_INVERSE_QUANTIZATION_MATRIX;
		isa.FrameArguments[isa.NumFrameArguments].Size = sizeof(input.QuantMtx);
		isa.FrameArguments[isa.NumFrameArguments].pData = &input.QuantMtx;
		++isa.NumFrameArguments;
	}

	isa.FrameArguments[isa.NumFrameArguments].Type = D3D12_VIDEO_DECODE_ARGUMENT_TYPE_SLICE_CONTROL;
	isa.FrameArguments[isa.NumFrameArguments].Size = input.SliceHeaders.Num() * sizeof(DXVA_Slice_HEVC_Short);
	isa.FrameArguments[isa.NumFrameArguments].pData = input.SliceHeaders.GetData();
	++isa.NumFrameArguments;
	isa.CompressedBitstream.pBuffer = fdr->D3DBitstreamBuffer;
	isa.CompressedBitstream.Offset = 0;
	isa.CompressedBitstream.Size = fdr->D3DBitstreamBufferPayloadSize;

	isa.ReferenceFrames.NumTexture2Ds = FFrameDecodeResource::kMaxRefFrames;
	isa.ReferenceFrames.ppTexture2Ds = fdr->ReferenceFrameList;
	isa.ReferenceFrames.pSubresources = fdr->ReferenceFrameListSubRes;
#if PLATFORM_WINDOWS
	isa.ReferenceFrames.ppHeaps = nullptr;
#endif

	IElectraDecoder::EDecoderError decres = ExecuteCommonDecode(isa, osa);
	if (decres != IElectraDecoder::EDecoderError::None)
	{
		// If not error, then pass on the list of outputs so far
		if (decres != IElectraDecoder::EDecoderError::Error)
		{
			if (HandleOutputListH265(OutputFrames) == IElectraDecoder::EDecoderError::Error)
			{
				decres = IElectraDecoder::EDecoderError::Error;
			}
		}
		return decres;
	}
	AutoRelease.ReleaseOwnership();

	// Add to the currently active list.
	fdr->D3DDecoder = VideoDecoder;
	fdr->D3DDecoderHeap = CurrentConfig.VideoDecoderHeap;
	//ActiveFrameDecodeResources.Emplace(MoveTemp(fdr));
	AvailableFrameDecodeResourceQueue.Enqueue(fdr);

	// Update the running frame number we use to associate this frame with.
	++RunningFrameNumLo;
	uint64 AssociatedUserValue = ((uint64)RunningFrameNumHi << 32) | (uint64)RunningFrameNumLo;

	// Create a new decoder output and set it up.
	TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe> InDec = MakeShared<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe>();
	InDec->PTS = InInputAccessUnit.PTS;
	InDec->UserValue = InInputAccessUnit.UserValue;
	InDec->OwningDPB = DPB;
	InDec->DecodedFrame = TargetFrame;
	InDec->UserValue0 = AssociatedUserValue;
	InDec->bDoNotOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) != EElectraDecoderFlags::None;
	InDec->OutputType = IElectraDecoderVideoOutput::EOutputType::Output;
	InSequenceParameterSet.GetCrop(InDec->Crop.Left, InDec->Crop.Right, InDec->Crop.Top, InDec->Crop.Bottom);

	InDec->Width = Align(InSequenceParameterSet.GetWidth(), GetFrameAlignment());
	InDec->Height = Align(InSequenceParameterSet.GetHeight(), GetFrameAlignment());
	// Adjust the cropping values to the right and bottom to include the required alignment we had to add.
	InDec->Crop.Right += InDec->Width - InSequenceParameterSet.GetWidth();
	InDec->Crop.Bottom += InDec->Height - InSequenceParameterSet.GetHeight();
	InDec->ImageWidth = InDec->Width - InDec->Crop.Left - InDec->Crop.Right;
	InDec->ImageHeight = InDec->Height - InDec->Crop.Top - InDec->Crop.Bottom;
	//InDec->Pitch = InDec->ImageWidth;
	InDec->NumBits = InSequenceParameterSet.bit_depth_luma_minus8 + 8;
	if (InDec->NumBits == 8)
	{
		InDec->BufferFormat = EPixelFormat::PF_NV12;
		InDec->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		InDec->ExtraValues.Emplace(TEXT("pixfmt"), FVariant((int64)EPixelFormat::PF_NV12));
		InDec->ExtraValues.Emplace(TEXT("pixenc"), FVariant((int64)EElectraTextureSamplePixelEncoding::Native));
	}
	else
	{
		InDec->BufferFormat = EPixelFormat::PF_P010;
		InDec->BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		InDec->ExtraValues.Emplace(TEXT("pixfmt"), FVariant((int64)EPixelFormat::PF_P010));
		InDec->ExtraValues.Emplace(TEXT("pixenc"), FVariant((int64)EElectraTextureSamplePixelEncoding::Native));
	}
	InSequenceParameterSet.GetAspect(InDec->AspectW, InDec->AspectH);
	auto FrameRate = InSequenceParameterSet.GetTiming();
	InDec->FrameRateN = FrameRate.Denom ? FrameRate.Num : 30;
	InDec->FrameRateD = FrameRate.Denom ? FrameRate.Denom : 1;
	InDec->Codec4CC = 'hvcC';
	InDec->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("dx")));
	InDec->ExtraValues.Emplace(TEXT("dxversion"), FVariant((int64) 12000));
	InDec->ExtraValues.Emplace(TEXT("sw"), FVariant(false));
	InDec->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("hevc")));
	FramesInDecoder.Emplace(MoveTemp(InDec));

	// Update the simulation DPB with the new decoded frame.
	ElectraDecodersUtil::MPEG::H265::FOutputFrameInfo FrameInfo;
	FrameInfo.IndexInBuffer = TargetFrame->IndexInPictureBuffer;
	FrameInfo.PTS = InInputAccessUnit.PTS;
	FrameInfo.UserValue0 = AssociatedUserValue;

	// Add this frame to the DPB. This may add additional frames to the output list.
	BitstreamParamsH265.DPB.AddDecodedFrame(OutputFrames, FrameInfo, InSliceInfos[0].Header);
	BitstreamParamsH265.bIsFirstInSequence = false;
	return HandleOutputListH265(OutputFrames);
}

IElectraDecoder::EDecoderError FD3D12VideoDecoder_H265::HandleOutputListH265(const TArray<ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::FDPBOutputFrame>& InOutputFrameInfos)
{
	TSharedPtr<FVideoDecoderOutputD3D12Electra, ESPMode::ThreadSafe> InDec;
	bool bUseDecodedOutput = true;
	for(int32 i=0; i<InOutputFrameInfos.Num(); ++i)
	{
		// In case the frame is a missing frame we ignore it.
		if (InOutputFrameInfos[i].UserFrameInfo.IndexInBuffer < 0)
		{
			continue;
		}

		//UE_LOGF(LogD3D12VideoDecodersElectra, Log, "Output frame %d, %lld", InOutputFrameInfos[i].UserFrameInfo.IndexInBuffer, (long long int)InOutputFrameInfos[i].UserFrameInfo.PTS.GetTicks());
		TSharedPtr<FDecodedFrame, ESPMode::ThreadSafe> Frame = DPB->GetFrameAtIndex(InOutputFrameInfos[i].UserFrameInfo.IndexInBuffer);
		check(Frame.IsValid());
		if (!Frame.IsValid())
		{
			PostError(0, FString::Printf(TEXT("HandleOutputListH265() failed. Output frame index is not valid for this DPB")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Locate the decoder output structure for this frame that we created earlier.
		for(int32 idx=0; idx<FramesInDecoder.Num(); ++idx)
		{
			if (FramesInDecoder[idx]->PTS == InOutputFrameInfos[i].UserFrameInfo.PTS)
			{
				bUseDecodedOutput = InOutputFrameInfos[i].bDoNotDisplay == false;
				InDec = FramesInDecoder[idx];
				FramesInDecoder.RemoveAt(idx);
				break;
			}
		}
		if (!InDec.IsValid())
		{
			PostError(0, FString::Printf(TEXT("HandleOutputListH265() failed. Output frame not found in input list")), ERRCODE_INTERNAL_FAILED_TO_DECODE);
			return IElectraDecoder::EDecoderError::Error;
		}
		check(InDec->OwningDPB == DPB);	// this should not trigger. A new DPB - if at all - should be created only when the decoder is flushed.
		check(InDec->DecodedFrame == Frame);
		check(InDec->UserValue0 == InOutputFrameInfos[i].UserFrameInfo.UserValue0);
		// Check if the upper layer decoder did not want this frame to be output.
		// This is different from the `DoNotOutput` flag we get from the DPB!
		if (!InDec->bDoNotOutput)
		{
			// Add to the ready-for-output queue.
			InDec->OutputType = bUseDecodedOutput ? IElectraDecoderVideoOutput::EOutputType::Output : IElectraDecoderVideoOutput::EOutputType::DoNotOutput;
			FramesReadyForOutput.Emplace(MoveTemp(InDec));
		}
		else
		{
			// Add to the queue of frames that were already output.
			// While this is not true we need to add it here and not back to the DPB because the
			// frame could still be referenced!
			FramesGivenOutForOutput.Emplace(MoveTemp(InDec));
		}
	}
	return IElectraDecoder::EDecoderError::None;
}


bool FD3D12VideoDecoder_H265::InternalResetToCleanStart()
{
	BitstreamParamsH265.Reset();
	return true;
}


}
