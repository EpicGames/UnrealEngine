// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoDecoder_D3D12_Common.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"

namespace ElectraVideoDecodersD3D12Video
{

class FD3D12VideoDecoder_H265 : public FD3D12VideoDecoder
{
public:
	FD3D12VideoDecoder_H265(const FCodecFormatHelper::FCodecInfo& InCodecInfo, const D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InDecodeSupport, const TMap<FString, FVariant>& InOptions, const TRefCountPtr<ID3D12Device>& InD3D12Device, const TRefCountPtr<ID3D12VideoDevice>& InVideoDevice, uint32 InVideoDeviceNodeIndex);
	virtual ~FD3D12VideoDecoder_H265();

protected:
	ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override;
	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;

private:
	bool InternalResetToCleanStart() override;

	struct FBitstreamParamsH265
	{
		void Reset()
		{
			VPSs.Empty();
			SPSs.Empty();
			PPSs.Empty();
			CurrentCSD.Empty();
			DPB.Reset();
			AssociatedIRAP = -1;
			IRAPCount = 0;
			bIsFirstInSequence = true;
		}
		struct FSliceDecodeInfo
		{
			uint8 NalUnitType = 0;
			uint8 NuhLayerId = 0;
			uint8 NumTemporalIdPlus1 = 0;

			// Parsed slice header.
			ElectraDecodersUtil::MPEG::H265::FSliceSegmentHeader Header;
			// Address of the nal unit byte of this slice.
			const uint8* NalUnitStartAddress = nullptr;
			// The number of bytes making up this slice, including the nal unit byte.
			uint32 NumBytesInSlice = 0;
		};

		TMap<uint32, ElectraDecodersUtil::MPEG::H265::FVideoParameterSet> VPSs;
		TMap<uint32, ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet> SPSs;
		TMap<uint32, ElectraDecodersUtil::MPEG::H265::FPictureParameterSet> PPSs;
		TArray<uint8> CurrentCSD;
		ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer DPB;
		int32 AssociatedIRAP = -1;
		int32 IRAPCount = 0;
		bool bIsFirstInSequence = true;
	};

	EDecoderError GetCodecSpecificDataH265(FBitstreamParamsH265& OutBitstreamParamsH265, const TMap<FString, FVariant>& InAdditionalOptions, bool bIsRequired);
	EDecoderError DecodeAccessUnitH265(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	EDecoderError DecodeSlicesH265(const FInputAccessUnit& InInputAccessUnit, const TArray<FBitstreamParamsH265::FSliceDecodeInfo>& InSliceInfos, const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& InSequenceParameterSet, const ElectraDecodersUtil::MPEG::H265::FPictureParameterSet& InPictureParameterSet);
	EDecoderError HandleOutputListH265(const TArray<ElectraDecodersUtil::MPEG::H265::FDecodedPictureBuffer::FDPBOutputFrame>& InOutputFrameInfos);

	FBitstreamParamsH265 BitstreamParamsH265;
	static const uint8 ScanOrderDiag4[16];
	static const uint8 ScanOrderDiag8[64];
	// Use the maximum MinCbSizeY value for image alignment as stipulated in the DXVA HEVC documentation.
	constexpr int32 GetFrameAlignment()
	{ return 64; }
};

}
