// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoder_D3D12_Common.h"

namespace ElectraVideoDecodersD3D12Video
{

bool FD3D12VideoDecoder::CheckPlatformDecodeCapabilities(D3D12_FEATURE_DATA_VIDEO_DECODE_SUPPORT& InOutDecodeSupport, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InOptions)
{
	return true;
}

}
