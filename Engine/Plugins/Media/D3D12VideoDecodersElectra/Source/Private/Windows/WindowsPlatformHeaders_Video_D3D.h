// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <CoreMinimal.h>

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <d3d11.h>
#include <d3d11_1.h>
#include <d3d12.h>
#include <d3d12video.h>
#include <dxgi1_4.h>
#include <dxva.h>

#include "mftransform.h"
#include "mfapi.h"
#include "mferror.h"
#include "mfidl.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END
